// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PreLoadScreen : ModuleRules
{
	public PreLoadScreen(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicDependencyModuleNames.AddRange(
			new string[] {
					"Engine",
					"ApplicationCore",
					"Analytics",
					"AnalyticsET",
				}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"InputCore",
					"RenderCore",
					"CoreUObject",
					"RHI",
					"Slate",
					"SlateCore",
					"BuildPatchServices",
					"Projects",
			}
		);


		//Need to make sure Android has Launch module so it can find and process AndroidEventManager events
		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Launch"
				}
			);
		}


		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
