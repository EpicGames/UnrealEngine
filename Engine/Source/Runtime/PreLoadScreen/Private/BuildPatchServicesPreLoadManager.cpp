// Copyright Epic Games, Inc. All Rights Reserved.
#include "BuildPatchServicesPreLoadManager.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "PreLoadManager.BuildPatchServices"

FBuildPatchServicesPreLoadManagerBase::FBuildPatchServicesPreLoadManagerBase()
    : bPatchingStarted(false)
    , bPatchingFinished(false)
{
}

void FBuildPatchServicesPreLoadManagerBase::Init()
{
    bPatchingStarted = false;
    bPatchingFinished = false;

    ContentBuildInstaller.Reset();
    BuildPatchServicesModule = &FModuleManager::LoadModuleChecked<IBuildPatchServicesModule>(TEXT("BuildPatchServices"));
}


bool FBuildPatchServicesPreLoadManagerBase::IsDone() const
{
    return bPatchingFinished;
}

void FBuildPatchServicesPreLoadManagerBase::StartBuildPatchServices(BuildPatchServices::FBuildInstallerConfiguration Settings)
{
	bPatchingStarted = true;

	if (ensureAlwaysMsgf(BuildPatchServicesModule, TEXT("FBuildPatchServicesPreLoadManager not initialized before install!")))
	{
		// Start the installer
		FBuildPatchInstallerDelegate BuildPatchInstallerDelegate = FBuildPatchInstallerDelegate::CreateRaw(this, &FBuildPatchServicesPreLoadManagerBase::OnContentBuildInstallerComplete);
		ContentBuildInstaller = BuildPatchServicesModule->CreateBuildInstaller(MoveTemp(Settings), MoveTemp(BuildPatchInstallerDelegate));
		ContentBuildInstaller->StartInstallation();
	}
	//If possible, at least try to still send the OnContentBuildInstallerComplete event if we ensured above
	else
	{
		bPatchingFinished = true;
		OnBuildPatchCompletedDelegate.Broadcast(false);
	}
}

void FBuildPatchServicesPreLoadManagerBase::OnContentBuildInstallerComplete(const IBuildInstallerRef& Installer)
{
    bPatchingFinished = true;
    OnBuildPatchCompletedDelegate.Broadcast(Installer->CompletedSuccessfully());
}

void FBuildPatchServicesPreLoadManagerBase::PauseBuildPatchInstall()
{
    if (ContentBuildInstaller.IsValid() && !ContentBuildInstaller->IsPaused())
    {
        ContentBuildInstaller->TogglePauseInstall();
    }
}

void FBuildPatchServicesPreLoadManagerBase::ResumeBuildPatchInstall()
{
    if (ContentBuildInstaller.IsValid() && ContentBuildInstaller->IsPaused())
    {
        ContentBuildInstaller->TogglePauseInstall();
    }
}

void FBuildPatchServicesPreLoadManagerBase::CancelBuildPatchInstall()
{
	if (ContentBuildInstaller.IsValid())
	{
		ContentBuildInstaller->CancelInstall();
	}
}

const FText& FBuildPatchServicesPreLoadManagerBase::GetStatusText() const
{
	// Static const fixed FText values so that they are not constantly constructed
	static const FText Queued = LOCTEXT("StatusText.Queued", "Queued");
	static const FText Initializing = LOCTEXT("StatusText.Initializing", "Initializing");
	static const FText Resuming = LOCTEXT("StatusText.Resuming", "Resuming");
	static const FText Downloading = LOCTEXT("StatusText.Downloading", "Downloading");
	static const FText Installing = LOCTEXT("StatusText.Installing", "Installing");
	static const FText BuildVerification = LOCTEXT("StatusText.BuildVerification", "Verifying");
	static const FText CleanUp = LOCTEXT("StatusText.CleanUp", "Cleaning up");
	static const FText PrerequisitesInstall = LOCTEXT("StatusText.PrerequisitesInstall", "Prerequisites");
	static const FText Completed = LOCTEXT("StatusText.Complete", "Complete");
	static const FText Paused = LOCTEXT("StatusText.Paused", "Paused");

	const BuildPatchServices::EBuildPatchState State = (ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetState() : BuildPatchServices::EBuildPatchState::Initializing);

	switch (State)
	{
	case BuildPatchServices::EBuildPatchState::Queued:
		return Queued;
	case BuildPatchServices::EBuildPatchState::Resuming:
		return Resuming;
	case BuildPatchServices::EBuildPatchState::Downloading:
		return Downloading;
	case BuildPatchServices::EBuildPatchState::Installing:
	case BuildPatchServices::EBuildPatchState::MovingToInstall:
	case BuildPatchServices::EBuildPatchState::SettingAttributes:
		return Installing;
	case BuildPatchServices::EBuildPatchState::BuildVerification:
		return BuildVerification;
	case BuildPatchServices::EBuildPatchState::CleanUp:
		return CleanUp;
	case BuildPatchServices::EBuildPatchState::PrerequisitesInstall:
		return PrerequisitesInstall;
	case BuildPatchServices::EBuildPatchState::Completed:
		return Completed;
	case BuildPatchServices::EBuildPatchState::Paused:
		return Paused;
	case BuildPatchServices::EBuildPatchState::Initializing:
	default:
		return Initializing;
	}
}

#undef LOCTEXT_NAMESPACE
