// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadScreenBase.h"

#include "Interfaces/IPluginManager.h"
#include "PreLoadSettingsContainer.h"

#include "Misc/ConfigCacheIni.h"

void FPreLoadScreenBase::InitSettingsFromConfig(const FString& ConfigFileName)
{	
	SCOPED_BOOT_TIMING("FPreLoadScreenBase::InitSettingsFromConfig");

    const FString UIConfigSection = "PreLoadScreen.UISettings";

    //Find plugin content path from name by going through enabled content plugins and see if one matches.
    FString PluginContentDir;
    for (TSharedRef<IPlugin> Plugin : IPluginManager::Get().GetEnabledPlugins())
    {
        if (Plugin->CanContainContent() && (Plugin->GetName().Equals(PluginName, ESearchCase::IgnoreCase)))
        {
            PluginContentDir = Plugin->GetContentDir();
            break;
        }
    }
    FPreLoadSettingsContainerBase::Get().SetPluginContentDir(PluginContentDir);

    FConfigFile* Config = GConfig->FindConfigFileWithBaseName(*ConfigFileName);
    if (ensureAlwaysMsgf(Config, TEXT("Unable to find .ini file for %s"), *ConfigFileName))
    {
        FPreLoadSettingsContainerBase* SettingsContainer = &FPreLoadSettingsContainerBase::Get();

        //Parse background display time
        float TimeToDisplayEachBackground = 5.0f;
        Config->GetFloat(*UIConfigSection, TEXT("TimeToDisplayEachBackground"), TimeToDisplayEachBackground);
    	FPreLoadSettingsContainerBase::Get().TimeToDisplayEachBackground = TimeToDisplayEachBackground;
    	
    	//Parse time to update the downloaded data size
    	float TimeToUpdateDownloadedDataSize = 0.0f;
    	Config->GetFloat(*UIConfigSection, TEXT("TimeToUpdateDownloadedDataSize"), TimeToUpdateDownloadedDataSize);
    	FPreLoadSettingsContainerBase::Get().TimeToUpdateDownloadedDataSize = TimeToUpdateDownloadedDataSize;
    	
        //Parse time to recheck notifications time
        float TimeToRecheckNotificationsPermission = 1.0f;
        Config->GetFloat(*UIConfigSection, TEXT("TimeToRecheckNotificationsPermission"), TimeToRecheckNotificationsPermission);
    	FPreLoadSettingsContainerBase::Get().TimeToRecheckNotificationsPermission = TimeToRecheckNotificationsPermission;
    	
    	//Parse time to show the notifications permission modal
    	float TimeToShowNotificationsPermissionModal = 2.0f;
    	Config->GetFloat(*UIConfigSection, TEXT("TimeToShowNotificationsPermissionModal"), TimeToShowNotificationsPermissionModal);
    	FPreLoadSettingsContainerBase::Get().TimeToShowNotificationsPermissionModal = TimeToShowNotificationsPermissionModal;
        
		//Parse LoadingGroups. You want to do this before ScreenGroupings and CustomImageBrushes
		TArray<FString> LoadingGroups;
		Config->GetArray(*UIConfigSection, TEXT("LoadingGroups"), LoadingGroups);
		SettingsContainer->ParseLoadingGroups(LoadingGroups);

        //Parse custom brushes
        TArray<FString> CustomImageBrushes;
        Config->GetArray(*UIConfigSection, TEXT("CustomImageBrushes"), CustomImageBrushes);
        for (const FString& BrushConfigEntry : CustomImageBrushes)
        {
            SettingsContainer->ParseBrushConfigEntry(BrushConfigEntry);
        }

        //Parse localized text
        TArray<FString> CustomLocalizedTexts;
        Config->GetArray(*UIConfigSection, TEXT("LocalizedText"), CustomLocalizedTexts);
        for (const FString& LocTextConfigEntry : CustomLocalizedTexts)
        {
            SettingsContainer->ParseLocalizedTextConfigString(LocTextConfigEntry);
        }
        
        //Parse ScreenGroupings
        TArray<FString> CustomScreenGroupings;
        Config->GetArray(*UIConfigSection, TEXT("ScreenGroupings"), CustomScreenGroupings);
        for (const FString& ScreenGroupingEntry : CustomScreenGroupings)
        {
            SettingsContainer->ParseScreenGroupingConfigString(ScreenGroupingEntry);            
        }

        //Parse Fonts
        TArray<FString> CustomFonts;
        Config->GetArray(*UIConfigSection, TEXT("CustomFont"), CustomFonts);
        for (const FString& FontEntry : CustomFonts)
        {
            SettingsContainer->ParseFontConfigEntry(FontEntry);
        }

		TArray<FString> ScreenOrders;
		Config->GetArray(*UIConfigSection, TEXT("ScreenOrders"), ScreenOrders);
		//Support old format of ScreenDisplayOrder=
		if (ScreenOrders.Num() == 0)
		{
			Config->GetArray(*UIConfigSection, TEXT("ScreenDisplayOrder"), ScreenOrders);
		}
		SettingsContainer->ParseAllScreenOrderEntries(LoadingGroups, ScreenOrders);

		SettingsContainer->PerformInitialAssetLoad();
    }
}

void FPreLoadScreenBase::CleanUp()
{
}

bool FPreLoadScreenBase::IsDone() const
{
    if (GetPreLoadScreenType() == EPreLoadScreenTypes::EngineLoadingScreen)
    {
        return bIsEngineLoadingFinished;
    }
    else
    {
        return !GetWidget().IsValid();
    }
}
