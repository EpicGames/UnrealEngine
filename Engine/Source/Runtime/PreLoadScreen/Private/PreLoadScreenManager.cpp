// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadScreenManager.h"

#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "GlobalShader.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"

#include "PreLoadScreen.h"
#include "PreLoadSettingsContainer.h"

#include "HAL/ThreadManager.h"
#include "Modules/ModuleManager.h"

#include "Stats/StatsSystem.h"

#if BUILD_EMBEDDED_APP
#include "Misc/EmbeddedCommunication.h"
#endif

#if PLATFORM_ANDROID
#if USE_ANDROID_EVENTS
#include "Android/AndroidEventManager.h"
#endif
#endif

#if PLATFORM_IOS
#include "IOS/IOSAsyncTask.h"
#endif

IMPLEMENT_MODULE(FDefaultModuleImpl, PreLoadScreen);

DEFINE_LOG_CATEGORY_STATIC(LogPreLoadScreenManager, Log, All);

TUniquePtr<FPreLoadScreenManager, FPreLoadScreenManager::FPreLoadScreenManagerDelete> FPreLoadScreenManager::Instance;
std::atomic<bool> FPreLoadScreenManager::bRenderingEnabled(true);

FCriticalSection FPreLoadScreenManager::ActivePreloadScreenCriticalSection;
TWeakPtr<IPreLoadScreen> FPreLoadScreenManager::ActivePreloadScreen;

FPreLoadScreenManager* FPreLoadScreenManager::Get()
{
	return Instance.Get();
}

void FPreLoadScreenManager::Create()
{
	check(IsInGameThread());

	if (!Instance.IsValid() && ArePreLoadScreensEnabled())
	{
		Instance = TUniquePtr<FPreLoadScreenManager, FPreLoadScreenManager::FPreLoadScreenManagerDelete>(new FPreLoadScreenManager);
	}
}

void FPreLoadScreenManager::Destroy()
{
	check(IsInGameThread());

	if (Instance.IsValid())
	{
		Instance->CleanUpResources();
		Instance.Reset();
	}
}

FPreLoadScreenManager::FPreLoadScreenManager()
	: ActivePreLoadScreenIndex(-1)
	, LastTickTime(0.0)
	, bInitialized(false)
	, SyncMechanism(nullptr)
	, bIsResponsibleForRendering(false)
	, bHasRenderPreLoadScreenFrame_RenderThread(false)
	, LastRenderTickTime(0.0)
	, OriginalSlateSleepVariableValue(0.f)
	, bIsEngineLoadingComplete(false)
{}

void FPreLoadScreenManager::OnRequestExit()
{
	if (Instance.IsValid() && ArePreLoadScreensEnabled())
	{
		RequestEngineExit(TEXT("Window closed during preload"));
	}
}

void FPreLoadScreenManager::Initialize(FSlateRenderer& InSlateRenderer)
{
	check(IsInGameThread());

	if (bInitialized || !ArePreLoadScreensEnabled())
	{
		return;
	}

    bInitialized = true;

	// Initialize shaders, because otherwise they might not be guaranteed to exist at this point
	if (!FPlatformProperties::RequiresCookedData() && GShaderCompilingManager)
	{
		TArray<int32> ShaderMapIds;
		ShaderMapIds.Add(GlobalShaderMapId);
		GShaderCompilingManager->FinishCompilation(TEXT("Global"), ShaderMapIds);
	}

	if (FApp::CanEverRender())
	{
		// Make sure we haven't created a game window already, if so use that. If not make a new one
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		TSharedRef<SWindow> GameWindow = (GameEngine && GameEngine->GameViewportWindow.IsValid()) ? GameEngine->GameViewportWindow.Pin().ToSharedRef() : UGameEngine::CreateGameWindow();

		VirtualRenderWindow =
			SNew(SVirtualWindow)
			.Size(GameWindow->GetClientSizeInScreen());

		MainWindow = GameWindow;

		WidgetRenderer = MakeShared<FPreLoadSlateWidgetRenderer, ESPMode::ThreadSafe>(GameWindow, VirtualRenderWindow, &InSlateRenderer);
	}

	LastRenderTickTime = FPlatformTime::Seconds();
	LastTickTime = FPlatformTime::Seconds();

	//This delegate will stay registered until another system do it. However, it will do something only for the lifetime of FPreLoadScreenManager.
	FSlateApplication::Get().SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&FPreLoadScreenManager::OnRequestExit));
}

void FPreLoadScreenManager::RegisterPreLoadScreen(const TSharedPtr<IPreLoadScreen>& PreLoadScreen)
{
	check(IsInGameThread());

    PreLoadScreens.Add(PreLoadScreen);
}

void FPreLoadScreenManager::UnRegisterPreLoadScreen(const TSharedPtr<IPreLoadScreen>& PreLoadScreen)
{
	check(IsInGameThread());

	if (PreLoadScreen.IsValid())
	{
		const int32 IndexOf = PreLoadScreens.IndexOfByKey(PreLoadScreen);
		if (PreLoadScreens.IsValidIndex(IndexOf))
		{
			if (IndexOf == ActivePreLoadScreenIndex)
			{
				ensureMsgf(false, TEXT("Can't remove an active preloadscreen."));
				return;
			}

			TSharedPtr<IPreLoadScreen> PreviousActivePreLoadScreen = PreLoadScreens.IsValidIndex(ActivePreLoadScreenIndex)
				? PreLoadScreens[ActivePreLoadScreenIndex] : nullptr;

			PreLoadScreen->CleanUp();
			PreLoadScreens.RemoveAtSwap(IndexOf);

			if (PreviousActivePreLoadScreen)
			{
				ActivePreLoadScreenIndex = PreLoadScreens.IndexOfByKey(PreviousActivePreLoadScreen);
			}
		}
	}
}

bool FPreLoadScreenManager::PlayFirstPreLoadScreen(EPreLoadScreenTypes PreLoadScreenTypeToPlay)
{
	for (int32 PreLoadScreenIndex = 0; PreLoadScreenIndex < PreLoadScreens.Num(); ++PreLoadScreenIndex)
	{
		if (PreLoadScreens[PreLoadScreenIndex]->GetPreLoadScreenType() == PreLoadScreenTypeToPlay)
		{
			PlayPreLoadScreenAtIndex(PreLoadScreenIndex);
			return true;
		}
	}
	return false;
}

void FPreLoadScreenManager::PlayPreLoadScreenAtIndex(int32 Index)
{
	check(IsInGameThread());

	if (IsEngineExitRequested())
	{	//Don't try to display a new preload screen when the engine is closing:
		//	-as we are closing we don't want to display load feedback, which is the opposite of what we are trying to do (but we cannot actually close in the middle of the init sequence).
		//	-we may no longer have a SWindow to use for display.
		return;
	}

	if (ArePreLoadScreensEnabled())
	{
		if (ensureAlwaysMsgf(!HasValidActivePreLoadScreen(), TEXT("Call to FPreLoadScreenManager::PlayPreLoadScreenAtIndex when something is already playing.")))
		{
			ActivePreLoadScreenIndex = Index;
			if (HasValidActivePreLoadScreen())
			{
				FScopeLock PreloadScreenLock(&ActivePreloadScreenCriticalSection);
				FPreLoadScreenManager::ActivePreloadScreen = PreLoadScreens[ActivePreLoadScreenIndex];
			}

			if (ensureAlwaysMsgf(HasValidActivePreLoadScreen(), TEXT("Call to FPreLoadScreenManager::PlayPreLoadScreenAtIndex with an invalid index! Nothing will play!")))
			{
				IPreLoadScreen* ActiveScreen = GetActivePreLoadScreen();
				if (ActiveScreen->GetPreLoadScreenType() == EPreLoadScreenTypes::EarlyStartupScreen)
				{
					HandleEarlyStartupPlay();
				}
				else if (ActiveScreen->GetPreLoadScreenType() == EPreLoadScreenTypes::EngineLoadingScreen)
				{
					HandleEngineLoadingPlay();
				}
				else if (ActiveScreen->GetPreLoadScreenType() == EPreLoadScreenTypes::CustomSplashScreen)
				{
					HandleCustomSplashScreenPlay();
				}
				else
				{
					UE_LOGF(LogPreLoadScreenManager, Fatal, "Attempting to play an Active PreLoadScreen type that hasn't been implemented inside of PreLoadScreenmanager!");
				}
			}
			else
			{
				{
					FScopeLock PreloadScreenLock(&ActivePreloadScreenCriticalSection);
					FPreLoadScreenManager::ActivePreloadScreen.Reset();
				}

				ActivePreLoadScreenIndex = INDEX_NONE;
			}
		}
	}
}

bool FPreLoadScreenManager::PlayPreLoadScreenWithTag(FName InTag)
{
	for (int32 PreLoadScreenIndex = 0; PreLoadScreenIndex < PreLoadScreens.Num(); ++PreLoadScreenIndex)
	{
		if (PreLoadScreens[PreLoadScreenIndex]->GetPreLoadScreenTag() == InTag)
		{
			PlayPreLoadScreenAtIndex(PreLoadScreenIndex);
			return true;
		}
	}
	return false;
}

void FPreLoadScreenManager::HandleEarlyStartupPlay()
{
	if (ensureAlwaysMsgf(HasActivePreLoadScreenType(EPreLoadScreenTypes::EarlyStartupScreen), TEXT("Invalid Active PreLoadScreen!")))
	{
		IPreLoadScreen* PreLoadScreen = GetActivePreLoadScreen();
		if (PreLoadScreen)
		{
			SCOPED_BOOT_TIMING("FPreLoadScreenManager::HandleEarlyStartupPlay()");

			PreLoadScreen->OnPlay(MainWindow);

			{
				TSharedPtr<SWindow> MainWindowPtr = MainWindow.Pin();
				if (MainWindowPtr.IsValid() && PreLoadScreen->GetWidget().IsValid())
				{
					MainWindowPtr->SetContent(PreLoadScreen->GetWidget().ToSharedRef());
				}
			}

			bool bDidDisableScreensaver = false;
			if (FPlatformApplicationMisc::IsScreensaverEnabled())
			{
				bDidDisableScreensaver = FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Disable);
			}

			FPlatformMisc::HidePlatformStartupScreen();

			{
				SCOPED_BOOT_TIMING("FPreLoadScreenManager::EarlyPlayFrameTick()");

				//We run this PreLoadScreen until its finished or we lose the MainWindow as EarlyPreLoadPlay is synchronous
				while (!PreLoadScreen->IsDone() && !IsEngineExitRequested())
				{
					EarlyPlayFrameTick();
				}
			}

			if (bDidDisableScreensaver)
			{
				FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Enable);
			}

			StopPreLoadScreen();
		}
	}
}

void FPreLoadScreenManager::HandleEngineLoadingPlay()
{
	if (ensureAlwaysMsgf(HasActivePreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen), TEXT("Invalid Active PreLoadScreen!")))
	{
		IPreLoadScreen* PreLoadScreen = GetActivePreLoadScreen();
		if (PreLoadScreen)
		{
			TSharedPtr<SWindow> MainWindowPtr = MainWindow.Pin();
			PreLoadScreen->OnPlay(MainWindowPtr);

			// The screen size may have changed between the VirtualRenderWindow creation and now
			TOptional<UE::Slate::FDeprecateVector2DResult> NewScreenSize;

			if (PreLoadScreen->GetWidget().IsValid() && VirtualRenderWindow.IsValid())
			{
				VirtualRenderWindow->SetContent(PreLoadScreen->GetWidget().ToSharedRef());

				if (MainWindowPtr.IsValid())
				{
					const UE::Slate::FDeprecateVector2DResult CurrentScreenSize = MainWindowPtr->GetClientSizeInScreen();
					if (VirtualRenderWindow->GetClientSizeInScreen() != CurrentScreenSize)
					{
						NewScreenSize.Emplace(CurrentScreenSize);
						VirtualRenderWindow->SetCachedSize(CurrentScreenSize);
					}
				}
			}

			//Need to update bIsResponsibleForRendering as a PreLoadScreen may not have updated it before this point
			if (!bIsResponsibleForRendering && PreLoadScreen->ShouldRender())
			{
				bIsResponsibleForRendering = true;
				IsResponsibleForRenderingDelegate.Broadcast(bIsResponsibleForRendering);

				if (NewScreenSize.IsSet() && FSlateApplication::IsInitialized() && MainWindowPtr.IsValid()) //MainWindowPtr should be valid if we get there, but... better safe than sorry.
				{
					// Force the viewport to resize before rendering
					FSlateApplication::Get().GetRenderer()->UpdateFullscreenState(MainWindowPtr.ToSharedRef(), (uint32)NewScreenSize.GetValue().X, (uint32)NewScreenSize.GetValue().Y);
				}
			}
		}

		if (WidgetRenderer.IsValid())
		{
			if (SyncMechanism == nullptr)
			{
				SyncMechanism = new FPreLoadScreenSlateSynchMechanism(WidgetRenderer);
				SyncMechanism->Initialize();
			}
		}
		}
	}

void FPreLoadScreenManager::HandleCustomSplashScreenPlay()
{
	if (ensureAlwaysMsgf(HasActivePreLoadScreenType(EPreLoadScreenTypes::CustomSplashScreen), TEXT("Invalid Active PreLoadScreen!")))
	{
		IPreLoadScreen* PreLoadScreen = GetActivePreLoadScreen();
		if (PreLoadScreen && MainWindow.IsValid())
		{
			SCOPED_BOOT_TIMING("FPreLoadScreenManager::HandleCustomSplashScreenPlay()");

			PreLoadScreen->OnPlay(MainWindow.Pin());

			if (PreLoadScreen->GetWidget().IsValid())
			{
				MainWindow.Pin()->SetContent(PreLoadScreen->GetWidget().ToSharedRef());
			}
			
			bool bDidDisableScreensaver = false;
			if (FPlatformApplicationMisc::IsScreensaverEnabled())
			{
				bDidDisableScreensaver = FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Disable);
			}

			FPlatformMisc::HidePlatformStartupScreen();
			FPlatformMisc::PlatformHandleSplashScreen(false);

			while (!PreLoadScreen->IsDone())
			{
				EarlyPlayFrameTick();
			}

			if (bDidDisableScreensaver)
			{
				FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Enable);
			}

			StopPreLoadScreen();
		}
	}
}

void FPreLoadScreenManager::StaticRenderTick_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	LLM_SCOPE(ELLMTag::RenderingThreadMemory);
	check(IsInRenderingThread());

	if (ensure(FPreLoadScreenManager::Get())) // The manager should clear the slate render thread before closing
	{
		FPreLoadScreenManager::Get()->RenderTick_RenderThread(RHICmdList);
	}
}

void FPreLoadScreenManager::RenderTick_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	//Calculate tick time
	const double CurrentTime = FPlatformTime::Seconds();
	float DeltaTime = static_cast<float>(CurrentTime - LastRenderTickTime);
	LastRenderTickTime = CurrentTime;

	FScopeLock PreloadScreenLock(&ActivePreloadScreenCriticalSection);
	TSharedPtr<IPreLoadScreen> PinnedActivePreloadScreen = FPreLoadScreenManager::ActivePreloadScreen.Pin();

	//Check if we have an active index before doing any work
	if (PinnedActivePreloadScreen && bRenderingEnabled)
	{
		if (MainWindow.IsValid() && VirtualRenderWindow.IsValid() && !PinnedActivePreloadScreen->IsDone())
		{
			GFrameNumberRenderThread++;
			PinnedActivePreloadScreen->RenderTick(RHICmdList, DeltaTime);
			RHICmdList.EndFrame();
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		}
	}
}

bool FPreLoadScreenManager::HasRegisteredPreLoadScreenType(EPreLoadScreenTypes PreLoadScreenTypeToCheck) const
{
    bool HasMatchingRegisteredScreen = false;
    for (const TSharedPtr<IPreLoadScreen>& Screen : PreLoadScreens)
    {
        if (Screen.IsValid() && (Screen->GetPreLoadScreenType() == PreLoadScreenTypeToCheck))
        {
            HasMatchingRegisteredScreen = true;
        }
    }

    return HasMatchingRegisteredScreen;
}

bool FPreLoadScreenManager::HasActivePreLoadScreenType(EPreLoadScreenTypes PreLoadScreenTypeToCheck) const
{
    return (HasValidActivePreLoadScreen() && (GetActivePreLoadScreen()->GetPreLoadScreenType() == PreLoadScreenTypeToCheck));
}

bool FPreLoadScreenManager::HasValidActivePreLoadScreen() const
{
    IPreLoadScreen* PreLoadScreen = nullptr;
    return (PreLoadScreens.IsValidIndex(ActivePreLoadScreenIndex) && PreLoadScreens[ActivePreLoadScreenIndex].IsValid());
}

IPreLoadScreen* FPreLoadScreenManager::GetActivePreLoadScreen()
{
    return HasValidActivePreLoadScreen() ? PreLoadScreens[ActivePreLoadScreenIndex].Get() : nullptr;
}

const IPreLoadScreen* FPreLoadScreenManager::GetActivePreLoadScreen() const
{
    return HasValidActivePreLoadScreen() ? PreLoadScreens[ActivePreLoadScreenIndex].Get() : nullptr;
}

bool FPreLoadScreenManager::HasActivePreLoadScreenTypeForEarlyStartup() const
{
	return HasActivePreLoadScreenType(EPreLoadScreenTypes::EarlyStartupScreen) || HasActivePreLoadScreenType(EPreLoadScreenTypes::CustomSplashScreen);
}

void FPreLoadScreenManager::EarlyPlayFrameTick()
{
    if (ensureAlwaysMsgf(HasActivePreLoadScreenTypeForEarlyStartup(), TEXT("EarlyPlayFrameTick called without a valid EarlyPreLoadScreen!")))
    {
        GameLogicFrameTick();
        EarlyPlayRenderFrameTick();
    }
}

void FPreLoadScreenManager::GameLogicFrameTick()
{
    IPreLoadScreen* ActivePreLoadScreen = GetActivePreLoadScreen();
    if (ensureAlwaysMsgf(ActivePreLoadScreen, TEXT("Invalid Active PreLoadScreen during GameLogicFameTick!")))
    {
        //First spin the platform by having it sleep a bit
        const float SleepTime = ActivePreLoadScreen ? ActivePreLoadScreen->GetAddedTickDelay() : 0.f;
        if (SleepTime > 0)
        {
            FPlatformProcess::Sleep(SleepTime);
        }

        double CurrentTime = FPlatformTime::Seconds();
        float DeltaTime = static_cast<float>(CurrentTime - LastTickTime);
        LastTickTime = CurrentTime;

		//Clamp to what should be more then any max reasonable time. This is to help with cases of
		//backgrounding or setting breakpoints to trigger huge ticks
		const float MaxTickTime = 5.0;
		DeltaTime = FMath::Min(DeltaTime, MaxTickTime);

        //We have to manually tick everything as we are looping the main thread here
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
        FTSTicker::GetCoreTicker().Tick(DeltaTime);
        FThreadManager::Get().Tick();

		//Tick any platform specific things we need here
		PlatformSpecificGameLogicFrameTick();

        //Tick the Active Screen
        ActivePreLoadScreen->Tick(DeltaTime);

        // Pump messages to handle input , etc from system
        FPlatformApplicationMisc::PumpMessages(true);

        FSlateApplication::Get().PollGameDeviceState();
        // Gives widgets a chance to process any accumulated input
        FSlateApplication::Get().FinishedInputThisFrame();

		FSlateApplication::Get().GetPlatformApplication()->Tick(DeltaTime);

        //Needed as this won't be incrementing on its own and some other tick functions rely on this (like analytics)
        GFrameCounter++;

		UE::Stats::FStats::AdvanceFrame(false);
    }
}

void FPreLoadScreenManager::PlatformSpecificGameLogicFrameTick()
{
#if PLATFORM_ANDROID
	Android_PlatformSpecificGameLogicFrameTick();
#endif //PLATFORM_ANDROID

#if PLATFORM_IOS
	IOS_PlatformSpecificGameLogicFrameTick();
#endif //PLATFORM_IOS
}

void FPreLoadScreenManager::EnableRendering(bool bEnabled)
{
    bRenderingEnabled = bEnabled;
}

void FPreLoadScreenManager::EarlyPlayRenderFrameTick()
{
	if (!bRenderingEnabled || !FSlateApplication::IsInitialized())
	{
		// If rendering disabled, FPreLoadScreenManager is responsible for rendering but choosing not to, probably because the
		// app is not in the foreground.
		FPlatformProcess::Sleep(0);
		return;
	}

	IPreLoadScreen* ActivePreLoadScreen = PreLoadScreens[ActivePreLoadScreenIndex].Get();
	if (ensureAlwaysMsgf(ActivePreLoadScreen, TEXT("Invalid Active PreLoadScreen during EarlyPlayRenderFrameTick!")))
	{
		bool bIsResponsibleForRendering_Local = true;
		if (!ActivePreLoadScreen->ShouldRender())
		{
			bIsResponsibleForRendering_Local = false;
		}

		if (bIsResponsibleForRendering_Local != bIsResponsibleForRendering)
		{
			bIsResponsibleForRendering = bIsResponsibleForRendering_Local;
			IsResponsibleForRenderingDelegate.Broadcast(bIsResponsibleForRendering);
		}

		if (bIsResponsibleForRendering_Local)
		{
			FSlateApplication& SlateApp = FSlateApplication::Get();
			float SlateDeltaTime = SlateApp.GetDeltaTime();

			//Setup Slate Render Command
			ENQUEUE_RENDER_COMMAND(BeginPreLoadScreenFrame)(
				[this, SlateDeltaTime](FRHICommandListImmediate& RHICmdList)
				{
					FScopeLock PreloadScreenLock(&ActivePreloadScreenCriticalSection);
					TSharedPtr<IPreLoadScreen> PinnedActivePreloadScreen = FPreLoadScreenManager::ActivePreloadScreen.Pin();

					// this is still valid because we do a FlushRenderingCommands in StopPreLoadScreen
					if (FPreLoadScreenManager::bRenderingEnabled && PinnedActivePreloadScreen && !bHasRenderPreLoadScreenFrame_RenderThread)
					{
						GFrameNumberRenderThread++;

						bHasRenderPreLoadScreenFrame_RenderThread = true;
						PinnedActivePreloadScreen->RenderTick(RHICmdList, SlateDeltaTime);
					}
				});

			SlateApp.Tick();

			// Synchronize the game thread and the render thread so that the render thread doesn't get too far behind.
			SlateApp.GetRenderer()->Sync();

			ENQUEUE_RENDER_COMMAND(FinishPreLoadScreenFrame)(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					// this is still valid because we do a FlushRenderingCommands in StopPreLoadScreen
					bHasRenderPreLoadScreenFrame_RenderThread = false;
					RHICmdList.EndFrame();
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
				});
		}
	}
}

void FPreLoadScreenManager::StopPreLoadScreen()
{
	check(IsInGameThread());

	if (HasValidActivePreLoadScreen())
	{
		if (ensureMsgf(HasActivePreLoadScreenTypeForEarlyStartup(), TEXT("WaitForEngineLoadingScreenToFinish should be called when using an EngineLoadingScreen.")))
		{
			HandleStopPreLoadScreen();
		}

		FlushRenderingCommands();
	}
}

void FPreLoadScreenManager::HandleStopPreLoadScreen()
{
	{
		if (HasValidActivePreLoadScreen())
		{
			PreLoadScreens[ActivePreLoadScreenIndex]->OnStop();
		}

		{
			FScopeLock PreloadScreenLock(&ActivePreloadScreenCriticalSection);
			FPreLoadScreenManager::ActivePreloadScreen.Reset();
		}

		ActivePreLoadScreenIndex = -1;

		//Clear our window content
		if (MainWindow.IsValid())
		{
			MainWindow.Pin()->SetContent(SNullWidget::NullWidget);
		}
		if (VirtualRenderWindow.IsValid())
		{
			VirtualRenderWindow->SetContent(SNullWidget::NullWidget);
		}
	}
}

void FPreLoadScreenManager::PassPreLoadScreenWindowBackToGame() const
{
	if (IsUsingMainWindow())
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		if (MainWindow.IsValid() && GameEngine)
		{
			GameEngine->GameViewportWindow = MainWindow;
		}
		else
		{
			UE_LOGF(LogPreLoadScreenManager, Warning, "FPreLoadScreenManager::PassLoadingScreenWindowBackToGame failed.  No Window");
		}
	}
}

bool FPreLoadScreenManager::IsUsingMainWindow() const
{
	return MainWindow.IsValid();
}

TSharedPtr<SWindow> FPreLoadScreenManager::GetRenderWindow()
{
	return MainWindow.IsValid() ? MainWindow.Pin() : nullptr;
}

void FPreLoadScreenManager::WaitForEngineLoadingScreenToFinish()
{
	check(IsInGameThread());

	//Start just doing game logic ticks until the Screen is finished.
	//Since this is a non-early screen, rendering happens separately still on the Slate rendering thread, so only need
	//the game logic ticks
	if (HasActivePreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
	{
		IPreLoadScreen* ActivePreLoadScreen = GetActivePreLoadScreen();

		//Add the current screens widget to focus so that any input gets funneled to it correctly.
		if (ActivePreLoadScreen)
		{
			TSharedPtr<SWidget> Widget = ActivePreLoadScreen->GetWidget();
			if (Widget.IsValid())
			{
				FSlateApplication::Get().SetAllUserFocus(Widget);
			}
		}

		while (ActivePreLoadScreen && !ActivePreLoadScreen->IsDone())
		{
			GameLogicFrameTick();
		}
	}

	//No longer need SyncMechanism now that the widget has finished rendering
	if (SyncMechanism != nullptr)
	{
		SyncMechanism->DestroySlateThread();

		delete SyncMechanism;
		SyncMechanism = nullptr;
	}

	HandleStopPreLoadScreen();
}

void FPreLoadScreenManager::SetEngineLoadingComplete(bool IsEngineLoadingFinished)
{
    bIsEngineLoadingComplete = IsEngineLoadingFinished;

    IPreLoadScreen* PreLoadScreen = GetActivePreLoadScreen();
    if (PreLoadScreen)
    {
        PreLoadScreen->SetEngineLoadingFinished(IsEngineLoadingFinished);
    }
}

bool FPreLoadScreenManager::ArePreLoadScreensEnabled()
{
	bool bEnabled = !GIsEditor && !IsRunningDedicatedServer() && !IsRunningCommandlet() && GUseThreadedRendering;

#if !UE_BUILD_SHIPPING
	bEnabled &= !FParse::Param(FCommandLine::Get(), TEXT("NoLoadingScreen"));
#endif

	return bEnabled;
}

void FPreLoadScreenManager::CleanUpResources()
{
	// Since we are on the game thread, the PreLoadScreen must be completed.
	//But if we are in EngineLoadingScreen, then the thread may be still active if WaitForEngineLoadingScreenToFinish was not called.
	bool bHasActiPreLoadScreen = HasValidActivePreLoadScreen();
	ensureMsgf(!bHasActiPreLoadScreen, TEXT("StopPreLoadScreen or WaitForEngineLoadingScreenToFinish (if EngineLoadingScreen) should be called before we destroy the Screen Manager."));

	if (SyncMechanism)
	{
		SyncMechanism->DestroySlateThread();
		delete SyncMechanism;
		SyncMechanism = nullptr;
	}

    for (TSharedPtr<IPreLoadScreen>& PreLoadScreen : PreLoadScreens)
    {
        if (PreLoadScreen.IsValid())
        {
            PreLoadScreen->CleanUp();
        }

        PreLoadScreen.Reset();
    }

    OnPreLoadScreenManagerCleanUp.Broadcast();

    //Make sure our FPreLoadSettingsContainer is cleaned up. We do this here instead of one of the
    //StartupScreens because we don't know how many of them will be using the same PreLoadScreenContainer, however any
    //other game specific settings containers should be cleaned up by their screens/modules
    BeginCleanup(&FPreLoadSettingsContainerBase::Get());
}

#if PLATFORM_ANDROID
void FPreLoadScreenManager::Android_PlatformSpecificGameLogicFrameTick()
{
#if USE_ANDROID_EVENTS
	// Process any Android events or we may have issues returning from background
	FAppEventManager::GetInstance()->Tick();
#endif //USE_ANDROIID_EVENTS
}
#endif //PLATFORM_ANDROID

#if PLATFORM_IOS
void FPreLoadScreenManager::IOS_PlatformSpecificGameLogicFrameTick()
{
	// drain the async task queue from the game thread
	[FAppleAsyncTask ProcessAsyncTasks];
}
#endif //PLATFORM_IOS
