// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadSettingsContainer.h"

#include "Brushes/SlateDynamicImageBrush.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/UnicodeBlockRange.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Framework/Application/SlateApplication.h"

#include "Internationalization/Culture.h"

FString FPreLoadSettingsContainerBase::UseSystemFontOverride = TEXT("SYSTEM");
FString FPreLoadSettingsContainerBase::DefaultInitialLoadingGroupIdentifier = TEXT("InitialLoad");
FPreLoadSettingsContainerBase* FPreLoadSettingsContainerBase::Instance = nullptr;

FPreLoadSettingsContainerBase::~FPreLoadSettingsContainerBase()
{
	for (auto& KVPair : BrushResources)
	{
		FSlateApplication::Get().GetRenderer()->ReleaseDynamicResource(*KVPair.Value);
		delete KVPair.Value;
	}
	BrushResources.Empty();
	LocalizedTextResources.Empty();
	FontResources.Empty();
	ScreenGroupings.Empty();

	Instance = nullptr;
}

void FPreLoadSettingsContainerBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& KVPair : BrushResources)
	{
		KVPair.Value->AddReferencedObjects(Collector);
	}
}

bool FPreLoadSettingsContainerBase::IsValidBrushConfig(TArray<FString>& SplitConfigEntry)
{
	return (SplitConfigEntry.Num() == 4);
}

void FPreLoadSettingsContainerBase::ParseBrushConfigEntry(const FString& ConfigEntry)
{
	if (ensureAlwaysMsgf(!ConfigEntry.IsEmpty(), TEXT("Attempt to parse empty ConfigEntry!")))
	{
		bool bWasValidEntry = true;

		TArray<FString> BrushComponents;
		ConfigEntry.ParseIntoArray(BrushComponents, TEXT("("), true);

		TArray<FString> BasicBrushComponents;
		BrushComponents[0].ParseIntoArray(BasicBrushComponents, TEXT(","), true);

		TArray<FString> LoadingGroupIdentifiers;

		if (BrushComponents.Num() == 1)
		{
			LoadingGroupIdentifiers.Add(DefaultInitialLoadingGroupIdentifier);
		}
		else if (BrushComponents.Num() == 2)
		{
			TArray<FString> LoadingIdentifiersForBrush;
			BrushComponents[1].ParseIntoArray(LoadingIdentifiersForBrush, TEXT(","), true);

			for (FString LoadingIdentifier : LoadingIdentifiersForBrush)
			{
				LoadingIdentifier.TrimStartAndEndInline();
				LoadingIdentifier.RemoveFromStart("(");
				LoadingIdentifier.RemoveFromEnd(")");
				LoadingIdentifier.RemoveFromEnd(")");

				LoadingGroupIdentifiers.Add(LoadingIdentifier);
			}
		}

		//Flag if we didn't find any LoadingIdentifiers or if our basic brush information is bad
		bWasValidEntry = bWasValidEntry && (LoadingGroupIdentifiers.Num() > 0) && IsValidBrushConfig(BasicBrushComponents);

		if (ensureAlwaysMsgf(bWasValidEntry, TEXT("Invalid Custom Brush in config. Exptected Format: +CustomImageBrushes=(Identifier,Filename,Width,Height,LoadGroupIdentifier) or +CustomImageBrushes=(Identifier,Filename,Width,Height). Config Entry: %s"), *ConfigEntry))
		{
			//Clean up the identifier to remove extra spaces and the first (
			FString Identifier = BasicBrushComponents[0];
			Identifier.TrimStartAndEndInline();
			Identifier.RemoveFromStart("(");

			FString FilePath = ConvertIfPluginRelativeContentPath(BasicBrushComponents[1]);

			float Width = FCString::Atof(*BasicBrushComponents[2]);
			float Height = FCString::Atof(*BasicBrushComponents[3]);

			FCustomBrushDefine NewBrushDefine(Identifier, FilePath, FVector2D(Width, Height));

			for (const FString& LoadingGroupIdentifier : LoadingGroupIdentifiers)
			{
				FCustomBrushLoadingGroup* FoundBrushLoadingGroup = BrushLoadingGroups.Find(*LoadingGroupIdentifier);
				if (ensureAlwaysMsgf((FoundBrushLoadingGroup != nullptr), TEXT("LoadingGroup not found for parsed LoadingGroupIdentifier %s. Config Entry: %s"), *LoadingGroupIdentifier, *ConfigEntry))
				{
					FoundBrushLoadingGroup->CustomBrushDefinesToLoad.Add(NewBrushDefine);
				}
			}
		}
	}
}

bool FPreLoadSettingsContainerBase::IsValidFontConfigString(TArray<FString>& SplitConfigEntry)
{
    return (SplitConfigEntry.Num() == 3);
}

void FPreLoadSettingsContainerBase::ParseFontConfigEntry(const FString& SplitConfigEntry)
{
    TArray<FString> FontComponents;
    SplitConfigEntry.ParseIntoArray(FontComponents, TEXT(","), true);
    if (ensureAlwaysMsgf(IsValidFontConfigString(FontComponents), TEXT("Invalid Font Entry in config: Expected Format: +CustomFont=(FontIdentifier, Language, FileName) Config Entry: %s"), *SplitConfigEntry))
    {
        FString Identifier = FontComponents[0];
        Identifier.TrimStartAndEndInline();
        Identifier.RemoveFromStart("(");

        FString Language = FontComponents[1];
        Language.TrimStartAndEndInline();

        FString FilePath = FontComponents[2];
        FilePath.TrimStartAndEndInline();
        FilePath.RemoveFromEnd(TEXT(")"));
		
		//Only convert pluging path if we aren't using the SystemFontOverride
		if (FilePath != UseSystemFontOverride)
		{
			FilePath = ConvertIfPluginRelativeContentPath(FilePath);
		}

        BuildCustomFont(Identifier, Language, FilePath);
    }
}

void FPreLoadSettingsContainerBase::ParseLoadingGroups(TArray<FString>& LoadingGroupIdentifiers)
{
	//if we have no loading groups, go ahead and add the default identifier
	if (LoadingGroupIdentifiers.Num() == 0)
	{
		ScreenOrderByLoadingGroups.FindOrAdd(*DefaultInitialLoadingGroupIdentifier);
		BrushLoadingGroups.FindOrAdd(*DefaultInitialLoadingGroupIdentifier);
	}
	else
	{
		for (const FString& LoadingGroupIdentifier : LoadingGroupIdentifiers)
		{
			ScreenOrderByLoadingGroups.FindOrAdd(*LoadingGroupIdentifier);
			BrushLoadingGroups.FindOrAdd(*LoadingGroupIdentifier);
		}
	}
}

void FPreLoadSettingsContainerBase::ParseAllScreenOrderEntries(TArray<FString>& LoadingGroupsEntries, TArray<FString>& ScreenOrderEntries)
{	
	for (const FString& ScreenOrderEntry : ScreenOrderEntries)
	{
		ParseScreenOrderConfigString(ScreenOrderEntry);
	}
}

void FPreLoadSettingsContainerBase::ParseScreenOrderConfigString(const FString& ScreenOrderEntry)
{
	TArray<FString> ScreenOrderComponents;
	ScreenOrderEntry.ParseIntoArray(ScreenOrderComponents, TEXT(","), true);
	
	FString LoadingGroupIdentifier;
	FString ScreenGroupingIdentifier;

	//Parse assuming default identifier and we are only supplying the MarketScreen
	if (ScreenOrderComponents.Num() == 1)
	{
		//We didn't supply an initial group identifier, so assume default
		LoadingGroupIdentifier = DefaultInitialLoadingGroupIdentifier;
		ScreenGroupingIdentifier = ScreenOrderComponents[0];
	}
	else if (ScreenOrderComponents.Num() == 2)
	{
		LoadingGroupIdentifier = ScreenOrderComponents[0];
		ScreenGroupingIdentifier = ScreenOrderComponents[1];
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Invalid ScreenOrder config entry! Expected format is either +ScreenOrders=(LoadingGroupsIdentifier,ScreenGroupingsIdentifier) or +ScreenOrders=(ScreenGroupingsIdentifier). Found: %s"), *ScreenOrderEntry);
		return;
	}

	//Clean up entries to remove extra spaces and parenthesis
	LoadingGroupIdentifier.RemoveSpacesInline();
	LoadingGroupIdentifier.RemoveFromStart("(");
	ScreenGroupingIdentifier.RemoveSpacesInline();
	ScreenGroupingIdentifier.RemoveFromStart("(");
	ScreenGroupingIdentifier.RemoveFromEnd(")");

	if (ensureAlwaysMsgf((!LoadingGroupIdentifier.IsEmpty() && !ScreenGroupingIdentifier.IsEmpty()), TEXT("No valid Loading Group Identifier or ScreenGroupingIdentifier found for ScreenOrderEntry! %s"), *ScreenOrderEntry))
	{
		FScreenOrderByLoadingGroup* FoundLoadingGroup = ScreenOrderByLoadingGroups.Find(*LoadingGroupIdentifier);
		FScreenGroupingBase* FoundScreenGrouping = ScreenGroupings.Find(*ScreenGroupingIdentifier);

		if (ensureAlwaysMsgf((FoundLoadingGroup != nullptr), TEXT("Did not find LoadingGroup definition for LoadingGroup %s for entry  %s"), *LoadingGroupIdentifier, *ScreenOrderEntry))
		{
			if (ensureAlwaysMsgf((FoundScreenGrouping != nullptr), TEXT("Did not find ScreenGrouping definition for ScreenGrouping %s for entry  %s"), *ScreenGroupingIdentifier, *ScreenOrderEntry))
			{
				FoundLoadingGroup->ScreenGroupings.Add(*ScreenGroupingIdentifier);
			}
		}
	}
}

void FPreLoadSettingsContainerBase::PerformInitialAssetLoad()
{
	//Try and load the default initial loading group, but only if we know we have it to avoid triggering any ensures
	FCustomBrushLoadingGroup* FoundDefaultLoadingGroup = BrushLoadingGroups.Find(*DefaultInitialLoadingGroupIdentifier);
	if (FoundDefaultLoadingGroup != nullptr)
	{
		LoadGrouping(*DefaultInitialLoadingGroupIdentifier);
	}
}

void FPreLoadSettingsContainerBase::LoadGrouping(FName Identifier)
{
	FCustomBrushLoadingGroup* FoundBrushLoadingGroup = BrushLoadingGroups.Find(*Identifier.ToString());
	if (ensureAlwaysMsgf((FoundBrushLoadingGroup != nullptr), TEXT("Could not find LoadGrouping for identifier:%s"), *Identifier.ToString()))
	{
		//only change if we can actually find the new LoadGrouping
		CurrentLoadGroup = Identifier;

		for (FCustomBrushDefine& BrushDefine : FoundBrushLoadingGroup->CustomBrushDefinesToLoad)
		{
			CreateCustomSlateImageBrush(BrushDefine.BrushIdentifier, BrushDefine.FilePath, BrushDefine.Size);
		}
	}
}

void FPreLoadSettingsContainerBase::BuildCustomFont(const FString& FontIdentifier, const FString& Language, const FString& FilePath)
{
	const bool bIsUsingSystemFont = (FilePath == UseSystemFontOverride);
	
	//Make sure we have created a System font file to load
	if (bIsUsingSystemFont)
	{
		BuildSystemFontFile();
	}

	//Try and find existing font, if we can't, make a new one
	TSharedPtr<FStandaloneCompositeFont>& FontToBuild = FontResources.FindOrAdd(*FontIdentifier);
	if (!FontToBuild.IsValid())
	{
		FontToBuild = MakeShared<FStandaloneCompositeFont>();
	}

	if (ensureAlwaysMsgf(FontToBuild.IsValid(), TEXT("Error creating custom font!")))
	{
		//Overwrite FilePath if using the system font
		FString FilePathToUse = bIsUsingSystemFont ? GetSystemFontFilePath() : FilePath;

		//If En, then setup as default font
		if (Language.Equals(TEXT("en")))
		{
			FontToBuild->DefaultTypeface.AppendFont(*FontIdentifier, FilePathToUse, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
		}
		// if not en, we need to setup some block ranges and subfonts to handle special characters
		else
		{
			TArray<EUnicodeBlockRange> BlockRangesForLanguage;
			//Arabic
			if (Language.Equals(TEXT("ar"), ESearchCase::IgnoreCase))
			{
				BlockRangesForLanguage.Add(EUnicodeBlockRange::Arabic);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::ArabicExtendedA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::ArabicMathematicalAlphabeticSymbols);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::ArabicPresentationFormsA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::ArabicPresentationFormsB);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::ArabicSupplement);
			}
			//Japanese
			else if (Language.Equals(TEXT("ja"), ESearchCase::IgnoreCase))
			{
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibility);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityForms);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKRadicalsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKStrokes);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKSymbolsAndPunctuation);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionB);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionC);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionD);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionE);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::EnclosedCJKLettersAndMonths);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::Hiragana);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::Katakana);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::KatakanaPhoneticExtensions);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::Kanbun);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HalfwidthAndFullwidthForms);
			}
			//Korean
			else if (Language.Equals(TEXT("ko"), ESearchCase::IgnoreCase))
			{
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HangulJamo);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HangulJamoExtendedA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HangulJamoExtendedB);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HangulCompatibilityJamo);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::HangulSyllables);
			}
			//Simplified Chinese
			else if (Language.Equals(TEXT("zh-hans"), ESearchCase::IgnoreCase))
			{
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibility);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityForms);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKRadicalsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKStrokes);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKSymbolsAndPunctuation);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionB);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionC);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionD);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionE);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::EnclosedCJKLettersAndMonths);
			}
			//Traditional Chinese
			else if (Language.Equals(TEXT("zh-hant"), ESearchCase::IgnoreCase))
			{
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibility);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityForms);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKCompatibilityIdeographsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKRadicalsSupplement);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKStrokes);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKSymbolsAndPunctuation);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographs);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionA);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionB);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionC);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionD);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::CJKUnifiedIdeographsExtensionE);
				BlockRangesForLanguage.Add(EUnicodeBlockRange::EnclosedCJKLettersAndMonths);
			}

			//Build out actual sub font ranges
			FCompositeSubFont& SubFont = FontToBuild->SubTypefaces[FontToBuild->SubTypefaces.AddDefaulted()];
			SubFont.Cultures.Append(Language);
			for (EUnicodeBlockRange& BlockRange : BlockRangesForLanguage)
			{
				SubFont.CharacterRanges.Add(FUnicodeBlockRange::GetUnicodeBlockRange(BlockRange).GetRange());
			}

			//Finally append actual font
			SubFont.Typeface.AppendFont(*FontIdentifier, FilePathToUse, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
		}
	}
}

const FString FPreLoadSettingsContainerBase::GetSystemFontFilePath() const
{
	return FPaths::EngineIntermediateDir() / TEXT("DefaultSystemFont.ttf");
}

bool FPreLoadSettingsContainerBase::BuildSystemFontFile()
{
	if (!HasCreatedSystemFontFile)
	{
		const TArray<uint8> FontBytes = FPlatformMisc::GetSystemFontBytes();
		if (FontBytes.Num() > 0)
		{			
			HasCreatedSystemFontFile = FFileHelper::SaveArrayToFile(FontBytes, *GetSystemFontFilePath());
		}
	}

	ensureAlwaysMsgf(HasCreatedSystemFontFile, TEXT("Failed to create system font!"));
	return HasCreatedSystemFontFile;
}

bool FPreLoadSettingsContainerBase::IsValidLocalizedTextConfigString(TArray<FString>& SplitConfigEntry)
{
    return (SplitConfigEntry.Num() >= 4);
}

void FPreLoadSettingsContainerBase::ParseLocalizedTextConfigString(const FString& ConfigEntry)
{
    TArray<FString> LocalizedTextComponents;
    ConfigEntry.ParseIntoArray(LocalizedTextComponents, TEXT(","), true);
    if (ensureAlwaysMsgf(IsValidLocalizedTextConfigString(LocalizedTextComponents), TEXT("Invalid Localized Text Entry in config: Expected Format: +LocalizedText=(TextIdentifier, NSLOCTEXT(Namespace, Key, Localized Text) or +LocalizedText=(TextIdentifier, Namespace, Key, Text) Config Entry: %s"), *ConfigEntry))
    {
        //Clean up the identifier to remove extra spaces and the first (
        FString Identifier = LocalizedTextComponents[0];
        Identifier.TrimStartAndEndInline();
        Identifier.RemoveFromStart("(");

        //LocalizedTextComponents[1] is the NameSpace for the loctext
        FString LocNameSpace = LocalizedTextComponents[1];
        LocNameSpace.TrimStartAndEndInline();
        LocNameSpace.RemoveFromStart("NSLOCTEXT(");
        LocNameSpace.RemoveFromStart("\"");
        LocNameSpace.RemoveFromEnd("\"");

        //LocalizedTextComponents[2] is the identifier for the FText
        FString LocIdentifier = LocalizedTextComponents[2];
        LocIdentifier.TrimStartAndEndInline();
        LocIdentifier.RemoveFromStart("\"");
        LocIdentifier.RemoveFromEnd("\"");

    	//LocalizedTextComponents[3;End] is the default text for the FText
		TArray<FString> LocInitialValueArray;
    	for (int32 i = 3; i < LocalizedTextComponents.Num(); ++i)
    	{
    		LocInitialValueArray.Add(LocalizedTextComponents[i]);
    	}
    	//Joining [3;End] with ',' in case the original FText contains ',' and has been split in LocalizedTextComponents.
    	FString LocInitialValue = FString::Join(LocInitialValueArray, TEXT(","));
        LocInitialValue.TrimStartAndEndInline();
        LocInitialValue.RemoveFromStart("\"");
        LocInitialValue.RemoveFromEnd(")"); //remove these separately so that if the file is missing one ) or the " is out of order it still works
        LocInitialValue.RemoveFromEnd(")");
        LocInitialValue.RemoveFromEnd("\"");

        //Add the FText to our list
        AddLocalizedText(Identifier, FText::AsLocalizable_Advanced(MoveTemp(LocNameSpace), MoveTemp(LocIdentifier), MoveTemp(LocInitialValue)));
    }
}

bool FPreLoadSettingsContainerBase::IsValidScreenGrooupingConfigString(TArray<FString>& SplitConfigEntry)
{
    return (SplitConfigEntry.Num() == 4);
}

void FPreLoadSettingsContainerBase::ParseScreenGroupingConfigString(const FString& ConfigEntry)
{
    TArray<FString> ScreenGroupingComponents;
    ConfigEntry.ParseIntoArray(ScreenGroupingComponents, TEXT(","), true);

    if (ensureAlwaysMsgf(IsValidScreenGrooupingConfigString(ScreenGroupingComponents), TEXT("Invalid ScreenGrouping Entry in config: Expected Format: +ScreenGrouping(ScreenIdentifier, Brush Identifier, Text Identifier, Font Size) Config Entry: %s"), *ConfigEntry))
    {
        //Clean up the identifier to remove extra spaces and the first (
        FString GroupIdentifier = ScreenGroupingComponents[0];
        GroupIdentifier.TrimStartAndEndInline();
        GroupIdentifier.RemoveFromStart("(");

        FString BrushIdentifier = ScreenGroupingComponents[1];
        BrushIdentifier.TrimStartAndEndInline();

        FString TextIdentifier = ScreenGroupingComponents[2];
        TextIdentifier.TrimStartAndEndInline();
        
        float FontSize = FCString::Atof(*ScreenGroupingComponents[3]);

        FScreenGroupingBase NewGrouping(BrushIdentifier, TextIdentifier, FontSize);
        AddScreenGrouping(GroupIdentifier, NewGrouping);
    }
}

FString FPreLoadSettingsContainerBase::ConvertIfPluginRelativeContentPath(const FString& FilePath)
{
    FString ReturnPath = FilePath.TrimStartAndEnd();
    if (!FPaths::FileExists(ReturnPath))
    {
        ReturnPath = PluginContentDir / ReturnPath;
    }

    ensureAlwaysMsgf(FPaths::FileExists(ReturnPath), TEXT("Can not find specified file %s"), *ReturnPath);
    return ReturnPath;
}

void FPreLoadSettingsContainerBase::SetShouldLoadBrushes(bool bInShouldLoadBrushes)
{
	bShouldLoadBrushes = bInShouldLoadBrushes;
}

void FPreLoadSettingsContainerBase::CreateCustomSlateImageBrush(const FString& Identifier, const FString& TexturePath, const FVector2D& ImageDimensions)
{
	if (bShouldLoadBrushes)
	{
		BrushResources.FindOrAdd(*Identifier, new FSlateDynamicImageBrush(*TexturePath, ImageDimensions));

		//Make sure this dynamic image resource is registered with the SlateApplication
		FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(*TexturePath);
	}
	else
	{
		BrushResources.FindOrAdd(*Identifier, new FSlateDynamicImageBrush(NAME_None, ImageDimensions));
	}
}

void FPreLoadSettingsContainerBase::AddLocalizedText(const FString& Identifier, FText LocalizedText)
{
    LocalizedTextResources.Add(*Identifier, LocalizedText);
}

void FPreLoadSettingsContainerBase::AddScreenGrouping(const FString& Identifier, FScreenGroupingBase& ScreenGrouping)
{
    ScreenGroupings.Add(*Identifier, ScreenGrouping);
}

const FSlateDynamicImageBrush* FPreLoadSettingsContainerBase::GetBrush(const FString& Identifier)
{
    const FSlateDynamicImageBrush*const* FoundBrush = BrushResources.Find(*Identifier);
    return FoundBrush ? *FoundBrush : nullptr;
}

FText FPreLoadSettingsContainerBase::GetLocalizedText(const FString& Identifier)
{
    const FText* FoundText = LocalizedTextResources.Find(*Identifier);
    return FoundText ? *FoundText : FText::GetEmpty();
}

TSharedPtr<FCompositeFont> FPreLoadSettingsContainerBase::GetFont(const FString& Identifier)
{
    TSharedPtr<FStandaloneCompositeFont>* FoundFontPointer = FontResources.Find(*Identifier);
    return FoundFontPointer ? *FoundFontPointer : TSharedPtr<FStandaloneCompositeFont>();
}

FPreLoadSettingsContainerBase::FScreenGroupingBase* FPreLoadSettingsContainerBase::GetScreenGrouping(const FString& Identifier)
{
    return ScreenGroupings.Find(*Identifier);
}

const FPreLoadSettingsContainerBase::FScreenGroupingBase* FPreLoadSettingsContainerBase::GetScreenAtIndex(int index) const
{ 
	return IsValidScreenIndex(index) ? ScreenGroupings.Find(ScreenOrderByLoadingGroups.Find(CurrentLoadGroup)->ScreenGroupings[index]) : nullptr;
}

bool FPreLoadSettingsContainerBase::IsValidScreenIndex(int index) const
{
	return (!CurrentLoadGroup.IsNone() && CurrentLoadGroup.IsValid() && (ScreenOrderByLoadingGroups.Find(CurrentLoadGroup) != nullptr)) ? ScreenOrderByLoadingGroups.Find(CurrentLoadGroup)->ScreenGroupings.IsValidIndex(index) : false;
}