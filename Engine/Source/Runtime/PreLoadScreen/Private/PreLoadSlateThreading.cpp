// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadSlateThreading.h"
#include "PreLoadScreenManager.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/Event.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Input/HittestGrid.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Rendering/SlateDrawBuffer.h"
#include "RenderingThread.h"
#include "Widgets/SVirtualWindow.h"


TAtomic<int32> FPreLoadScreenSlateSynchMechanism::LoadingThreadInstanceCounter(0);

bool FPreLoadScreenSlateThreadTask::Init()
{
	// First thing to do is set the slate loading thread ID
	// This guarantees all systems know that a slate thread exists
	const int32 PreviousValue = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, FPlatformTLS::GetCurrentThreadId(), 0);

	bool bSuccess = PreviousValue == 0;
	ensureMsgf(bSuccess, TEXT("Only one system can use the SlateThread at the same time. PreLoadScreen is not compatible with GetMoviePlayer."));
	return bSuccess;
}

uint32 FPreLoadScreenSlateThreadTask::Run()
{
	FTaskTagScope Scope(ETaskTag::ESlateThread);
	check(GSlateLoadingThreadId == FPlatformTLS::GetCurrentThreadId());

	SyncMechanism->RunMainLoop_SlateThread();

	return 0;
}

void FPreLoadScreenSlateThreadTask::Exit()
{
	// Tear down the slate loading thread ID
	const int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	const int32 PreviousThreadId = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, 0, CurrentThreadId);
	check(PreviousThreadId == CurrentThreadId);
}

FPreLoadSlateWidgetRenderer::FPreLoadSlateWidgetRenderer(TSharedPtr<SWindow> InMainWindow, TSharedPtr<SVirtualWindow> InVirtualRenderWindow, FSlateRenderer* InRenderer)
	: MainWindow(InMainWindow.Get())
	, VirtualRenderWindow(InVirtualRenderWindow.ToSharedRef())
	, SlateRenderer(InRenderer)
{
    HittestGrid = MakeShareable(new FHittestGrid);
}

void FPreLoadSlateWidgetRenderer::DrawWindow(float DeltaTime)
{
	// If the engine has requested exit, early out the draw loop, several
	// of the things inside of here are not safe to perform while shutting down.
	if (IsEngineExitRequested())
	{
		return;
	}

	// FPreLoadScreenSlateSynchMechanism need to be shutdown before we release the FSlateApplication
	if (ensure(FSlateApplication::IsInitialized()))
	{
		FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
		FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());

		const FVector2D DrawSize = VirtualRenderWindow->GetClientSizeInScreen();

		FSlateApplication::Get().Tick(ESlateTickType::Time);

		const float Scale = 1.0f;
		FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize, FSlateLayoutTransform(Scale));

		VirtualRenderWindow->SlatePrepass(WindowGeometry.Scale);

		FSlateRect ClipRect = WindowGeometry.GetLayoutBoundingRect();

		HittestGrid->SetHittestArea(VirtualRenderWindow->GetPositionInScreen(), VirtualRenderWindow->GetViewportSize());
		HittestGrid->Clear();

		{
			// Get the free buffer & add our virtual window
			FSlateRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *SlateRenderer };
			FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(VirtualRenderWindow);

			WindowElementList.SetRenderTargetWindow(MainWindow);

			int32 MaxLayerId = 0;
			{
				FPaintArgs PaintArgs(nullptr, *HittestGrid, FVector2D::ZeroVector, FSlateApplication::Get().GetCurrentTime(), FSlateApplication::Get().GetDeltaTime());

				// Paint the window
				MaxLayerId = VirtualRenderWindow->Paint(
					PaintArgs,
					WindowGeometry, ClipRect,
					WindowElementList,
					0,
					FWidgetStyle(),
					VirtualRenderWindow->IsEnabled());
			}

			SlateRenderer->DrawWindows(ScopedDrawBuffer.GetDrawBuffer());
			ScopedDrawBuffer.GetDrawBuffer().ViewOffset = FVector2D::ZeroVector;
		}
	}
}


FPreLoadScreenSlateSynchMechanism::FPreLoadScreenSlateSynchMechanism(TSharedPtr<FPreLoadSlateWidgetRenderer, ESPMode::ThreadSafe> InWidgetRenderer)
    : bIsRunningSlateMainLoop(false)
	, SlateLoadingThread(nullptr)
	, SlateRunnableTask(nullptr)
	, SleepEvent(nullptr)
	, WidgetRenderer(InWidgetRenderer)
{
}

FPreLoadScreenSlateSynchMechanism::~FPreLoadScreenSlateSynchMechanism()
{
    DestroySlateThread();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnWindowBeingDestroyed().RemoveAll(this);
		FSlateApplication::Get().OnPreShutdown().RemoveAll(this);
	}
}

void FPreLoadScreenSlateSynchMechanism::Initialize()
{
    check(IsInGameThread());

	// Initialize should only be called once
	if (ensure(FSlateApplication::IsInitialized() && bIsRunningSlateMainLoop == 0))
	{
		// Prevent the application from closing while we are rendering from another thread
		FSlateApplication::Get().OnPreShutdown().AddRaw(this, &FPreLoadScreenSlateSynchMechanism::DestroySlateThread);
		FSlateApplication::Get().OnWindowBeingDestroyed().AddRaw(this, &FPreLoadScreenSlateSynchMechanism::HandleWindowBeingDestroyed);

		bIsRunningSlateMainLoop = true;
		check(SlateLoadingThread == nullptr);

		TStringBuilder<32> ThreadName;
		ThreadName.Append(TEXT("SlateLoadingThread"));
		ThreadName.Appendf(TEXT("%d"), LoadingThreadInstanceCounter.Load());
		LoadingThreadInstanceCounter++;

		SleepEvent = FGenericPlatformProcess::GetSynchEventFromPool(false);
		SlateRunnableTask = new FPreLoadScreenSlateThreadTask(*this);
		SlateLoadingThread = FRunnableThread::Create(SlateRunnableTask, ThreadName.GetData());
	}
}

void FPreLoadScreenSlateSynchMechanism::DestroySlateThread()
{
	check(IsInGameThread());

	if (bIsRunningSlateMainLoop)
	{
		check(SlateLoadingThread != nullptr);
		bIsRunningSlateMainLoop = false;
		SleepEvent->Trigger();
		SlateLoadingThread->WaitForCompletion();

		FGenericPlatformProcess::ReturnSynchEventToPool(SleepEvent);
		delete SlateLoadingThread;
		delete SlateRunnableTask;
		SlateLoadingThread = nullptr;
		SlateRunnableTask = nullptr;
	}
}

void FPreLoadScreenSlateSynchMechanism::HandleWindowBeingDestroyed(const SWindow& WindowBeingDestroyed)
{
	check(IsInGameThread());

	if (WidgetRenderer && WidgetRenderer->GetMainWindow_GameThread() == &WindowBeingDestroyed)
	{
		// wait until the render has completed its task
		DestroySlateThread();
	}
}

bool FPreLoadScreenSlateSynchMechanism::IsSlateMainLoopRunning_AnyThread() const
{
    return bIsRunningSlateMainLoop && !IsEngineExitRequested();
}

void FPreLoadScreenSlateSynchMechanism::RunMainLoop_SlateThread()
{
	double LastTime = FPlatformTime::Seconds();
	bool bManualReset = false;
	FEvent* EnqueueRenderEvent = FGenericPlatformProcess::GetSynchEventFromPool(bManualReset);

	while (IsSlateMainLoopRunning_AnyThread())
	{
		// Test to ensure that we are still the SlateLoadingThread
		checkCode(
			const int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			const int32 PreviousThreadId = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, CurrentThreadId, CurrentThreadId);
			check(PreviousThreadId == CurrentThreadId)
		);

		double CurrentTime = FPlatformTime::Seconds();
		double DeltaTime = CurrentTime - LastTime;
		{
			// 60 fps max
			const double MaxTickRate = 1.0 / 60.0f;
			double TimeToWait = MaxTickRate - DeltaTime;

			if (TimeToWait > 0.0)
			{
				SleepEvent->Wait(FTimespan::FromSeconds(TimeToWait));
				CurrentTime = FPlatformTime::Seconds();
				DeltaTime = CurrentTime - LastTime;
			}
			LastTime = CurrentTime;
		}

		bool bRenderCommandEnqeued = false;
		if (IsSlateMainLoopRunning_AnyThread())
		{
			// This avoids crashes if we Suspend rendering whilst the loading screen is up
			// as we don't want Slate to submit any more draw calls until we Resume.
			if (GDynamicRHI && !GDynamicRHI->RHIIsRenderingSuspended())
			{
				if (IsSlateMainLoopRunning_AnyThread() && FPreLoadScreenManager::bRenderingEnabled)
				{
					WidgetRenderer->DrawWindow(static_cast<float>(DeltaTime));

					bRenderCommandEnqeued = true;

					//Queue up a render tick every time we tick on this sync thread.
					FPreLoadScreenSlateSynchMechanism* SyncMech = this;
					ENQUEUE_RENDER_COMMAND(PreLoadScreenRenderTick)(
						[SyncMech, EnqueueRenderEvent](FRHICommandListImmediate& RHICmdList)
						{
							if (SyncMech->IsSlateMainLoopRunning_AnyThread())
							{
								FPreLoadScreenManager::StaticRenderTick_RenderThread(RHICmdList);
							}

							EnqueueRenderEvent->Trigger();
						}
					);
				}
			}
		}

		if (bRenderCommandEnqeued)
		{
			// Release the lock and wait for the enqueued command to complete.
			//Only one command at the time. Don't Flush because other commands might be less important and we are in the loading phase.
			EnqueueRenderEvent->Wait();
		}
	}

	FGenericPlatformProcess::ReturnSynchEventToPool(EnqueueRenderEvent);
}
