// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimplePreLoadScreenWidget.h"
#include "PreLoadSettingsContainer.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

FCriticalSection SSimplePreLoadScreenWidget::BackgroundImageCrit;
int SSimplePreLoadScreenWidget::CurrentBackgroundImage = 0;

void SSimplePreLoadScreenWidget::Construct(const FArguments& InArgs)
{
    TimeToDisplayEachBackground = FPreLoadSettingsContainerBase::Get().TimeToDisplayEachBackground;

    ChildSlot
    [
        SNew(SDPIScaler)
        .DPIScale(this, &SSimplePreLoadScreenWidget::GetDPIScale)
        [
            SNew(SOverlay)

            +SOverlay::Slot()
            .HAlign(HAlign_Fill)
            .VAlign(VAlign_Fill)
            [
                SNew(SVerticalBox)
                    
                + SVerticalBox::Slot()
                [
                    SNew(SOverlay)

                    //Background Display
                    +SOverlay::Slot()
                    [
                        SNew(SScaleBox)
                        .Stretch(EStretch::ScaleToFit)
                        [
                            SNew(SImage)
                            .Image(this, &SSimplePreLoadScreenWidget::GetCurrentBackgroundImage)
                        ]
                    ]
                    
                    //Simple Text Display
                    +SOverlay::Slot()
                    [
                        SNew(SVerticalBox)
                        + SVerticalBox::Slot()
                        .FillHeight(.82f)
                        [
                            //Empty space waster just to take up most of the screen
                            SNew(SBox)
                        ]
                        
                        //Text display on the bottom center of the screen
                        + SVerticalBox::Slot()
                        .FillHeight(.18f)
                        [                            
                            SNew(SBox)
                            .VAlign(VAlign_Center)
                            .HAlign(HAlign_Center)
                            .Padding(FMargin(50.f,5.f))
                            [     
                                SNew(SScaleBox)
                                .Stretch(EStretch::ScaleToFit)
                                [
                                    SNew(STextBlock)
                                    .Justification(ETextJustify::Center)
                                    .Font(this, &SSimplePreLoadScreenWidget::GetTextFont)
                                    .Text(this, &SSimplePreLoadScreenWidget::GetCurrentScreenText)
                                    .ColorAndOpacity(FColor::White)
                                ]
                            ]
                        ]
                    ]
                ]
            ]
        ]
    ];
}


const FSlateBrush* SSimplePreLoadScreenWidget::GetCurrentBackgroundImage() const
{
    FScopeLock ScopeLock(&BackgroundImageCrit);

    const FPreLoadSettingsContainerBase::FScreenGroupingBase* CurrentScreenIdentifier = FPreLoadSettingsContainerBase::Get().GetScreenAtIndex(CurrentBackgroundImage);
    const FString& BackgroundBrushIdentifier = CurrentScreenIdentifier ? CurrentScreenIdentifier->ScreenBackgroundIdentifer : TEXT("");
    return FPreLoadSettingsContainerBase::Get().GetBrush(BackgroundBrushIdentifier);
}

FText SSimplePreLoadScreenWidget::GetCurrentScreenText() const
{
    FScopeLock ScopeLock(&BackgroundImageCrit);
    const FPreLoadSettingsContainerBase::FScreenGroupingBase* CurrentScreenIdentifier = FPreLoadSettingsContainerBase::Get().GetScreenAtIndex(CurrentBackgroundImage);
    const FString& TextIdentifier = CurrentScreenIdentifier ? CurrentScreenIdentifier->TextIdentifier : TEXT("");
    return FPreLoadSettingsContainerBase::Get().GetLocalizedText(TextIdentifier);
}

FSlateFontInfo SSimplePreLoadScreenWidget::GetTextFont() const
{
    const FPreLoadSettingsContainerBase::FScreenGroupingBase* CurrentScreenIdentifier = FPreLoadSettingsContainerBase::Get().GetScreenAtIndex(CurrentBackgroundImage);
    const float FontSize = CurrentScreenIdentifier ? CurrentScreenIdentifier->FontSize : 0.0f;

    return GetFontInfo("Main-new", FontSize);
}

FSlateFontInfo SSimplePreLoadScreenWidget::GetFontInfo(const FString& FontName, float FontSize) const
{
    return FSlateFontInfo(FPreLoadSettingsContainerBase::Get().GetFont(FontName), FontSize);
}