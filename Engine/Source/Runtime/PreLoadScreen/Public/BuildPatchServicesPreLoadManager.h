// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuildPatchSettings.h"
#include "BuildPatchState.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class IBuildPatchServicesModule;

//This class is used to help manage a PreLoadScreen based on a BuildPatchServices install.
class FBuildPatchServicesPreLoadManagerBase : public TSharedFromThis<FBuildPatchServicesPreLoadManagerBase>
{
public:
    PRELOADSCREEN_API FBuildPatchServicesPreLoadManagerBase();
    virtual ~FBuildPatchServicesPreLoadManagerBase() = default;

    PRELOADSCREEN_API virtual void Init();
    
    //Setup BPT with everything now loaded
    PRELOADSCREEN_API virtual void StartBuildPatchServices(BuildPatchServices::FBuildInstallerConfiguration Settings);

    //BPT finished
    PRELOADSCREEN_API virtual void OnContentBuildInstallerComplete(const IBuildInstallerRef& Installer);

    PRELOADSCREEN_API virtual bool IsDone() const;

    virtual int64 GetDownloadSize() { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetTotalDownloadRequired() : 0; }
    virtual int64 GetDownloadProgress() { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetTotalDownloaded() : 0; }
    
    DECLARE_MULTICAST_DELEGATE_OneParam(FOnBuildPatchCompleted, bool );
    FOnBuildPatchCompleted OnBuildPatchCompletedDelegate;

    PRELOADSCREEN_API virtual void PauseBuildPatchInstall();
    PRELOADSCREEN_API virtual void ResumeBuildPatchInstall();
	PRELOADSCREEN_API virtual void CancelBuildPatchInstall();

    float GetProgressPercent() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetUpdateProgress() : 0.0f; }
    EBuildPatchDownloadHealth GetDownloadHealth() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetDownloadHealth() : EBuildPatchDownloadHealth::NUM_Values; }
    PRELOADSCREEN_API const FText& GetStatusText() const;
    BuildPatchServices::EBuildPatchState GetState() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetState() : BuildPatchServices::EBuildPatchState::Initializing; }

    FText GetErrorMessageBody() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetErrorText() : FText(); }
    EBuildPatchInstallError GetErrorType() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetErrorType() : EBuildPatchInstallError::NoError; }
    FString GetErrorCode() const { return ContentBuildInstaller.IsValid() ? ContentBuildInstaller->GetErrorCode() : TEXT("U"); }

    virtual bool IsActive()
    {
        return (ContentBuildInstaller.IsValid()
            && !ContentBuildInstaller->IsComplete()
            && !ContentBuildInstaller->HasError()
            && ContentBuildInstaller->GetState() > BuildPatchServices::EBuildPatchState::Resuming);
    }

    IBuildInstallerPtr GetInstaller() { return ContentBuildInstaller; }

protected:
    bool bPatchingStarted;
    bool bPatchingFinished;

    IBuildPatchServicesModule* BuildPatchServicesModule;
    IBuildInstallerPtr ContentBuildInstaller;
};
