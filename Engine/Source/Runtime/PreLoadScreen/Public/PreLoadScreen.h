// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

#include "IAnalyticsProviderET.h"

enum class EPreLoadScreenTypes : uint8
{
	CustomSplashScreen,
    EarlyStartupScreen,
    EngineLoadingScreen
};

// Interface that defines the class that handles all the logic for controlling / displaying a particular PreLoadScreen.
// Designed to be implemented in a Plugin that calls FPreLoadScreenManager::RegisterPreLoadScreen so that functions are called by PreLoadScreenManager correctly.
// Really should probably inherit from FPreLoadScreenBase instead of this class for more functionality
class IPreLoadScreen : public TSharedFromThis<IPreLoadScreen>
{
public:
	virtual ~IPreLoadScreen() = default;

	virtual void Init() = 0;

	/** Standard tick that happens every frame */
	virtual void Tick(float DeltaTime) = 0;

	/**
	 * This function is used to determine if an extra platform sleep should be performed every tick (to slow down the tick rate)
	 * keeps us from spinning super fast when we aren't doing much beyond loading data / etc on other threads.
	 */
	virtual float GetAddedTickDelay() = 0;

	/** Whether an EarlyStartupLoadScreen should render */
	virtual bool ShouldRender() const = 0;

	/**
	 * This tick happens as part of the slate render tick during an EarlyStartupLoadScreen
	 * @note Called on the render thread
	 */
	virtual void RenderTick(FRHICommandListImmediate& RHICmdList, float DeltaTime) = 0;

	/** Callback for when a PreLoadScreen starts being displayed. Provides a reference to the SWindow that will be used to display content */
	virtual void OnPlay(TWeakPtr<SWindow> TargetWindow) = 0;

	/** Callback for when a PreLoadScreen is no longer being displayed. */
	virtual void OnStop() = 0;

	/**
	 * Returns true when the PreLoadScreen is completed.
	 * @note This need to be thread safe
	 */
	virtual bool IsDone() const = 0;

	virtual void CleanUp() = 0;

	/**
	 * Should override this function to determine if this screen should be used to handle EarlyStartupScreen behavior
	 * @note IMPORTANT: This changes a LOT of functionality and implementation details. EarlyStartupScreens happen before the engine is fully initialized and block engine initialization before they finish.
	 *                  this means they have to forgo even the most basic of engine features like UObject support, as they are displayed before those systems are initialized.
	 */
	virtual EPreLoadScreenTypes GetPreLoadScreenType() const = 0;

	/**
	 * Allows the PreLoadScreen to register a tag that can be later used to find a specific loading screen. 
	 * PreLoadScreens not using this functionality should return NAME_None
	 */
	virtual FName GetPreLoadScreenTag() const = 0;

	/** @See IsDone when GetPreLoadScreenType() returns EPreLoadScreenTypes::EngineLoadingScreen */
	virtual void SetEngineLoadingFinished(bool IsEngineLoadingFinished) = 0;

	virtual const TSharedPtr<const SWidget> GetWidget() const = 0;
	virtual TSharedPtr<SWidget> GetWidget() = 0;
};