// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreLoadScreen.h"

// Base implementation of the IPreLoadScreen that handles all the logic for controlling / updating the UI for PreLoadScreens.
// Designed to be overriden by a game specific Plugin that calls FPreloadScreenManager::RegisterPreLoadScreen so that functions are called by the PreLoadScreenManager correctly.
class FPreLoadScreenBase : public IPreLoadScreen 
{

    /**** IPreLoadScreen implementation ****/
public:
    //We don't use these in the FPreLoadScreenBase, but they are useful for game-specific implementations
    virtual void Tick(float DeltaTime) override {}
	virtual bool ShouldRender() const override { return true; }
    virtual void RenderTick(FRHICommandListImmediate& RHICmdList, float DeltaTime) override {};
    virtual void OnStop() override {}

    //Store off TargetWindow
    virtual void OnPlay(TWeakPtr<SWindow> TargetWindow) override { OwningWindow = TargetWindow; }

    //By default have a small added tick delay so we don't super spin out while waiting on other threads to load data / etc.
    virtual float GetAddedTickDelay() override { return 0.00f; }

	virtual void Init() override {}
    
    virtual TSharedPtr<SWidget> GetWidget() override { return nullptr; }
    virtual const TSharedPtr<const SWidget> GetWidget() const override { return nullptr; }

    //IMPORTANT: This changes a LOT of functionality and implementation details. EarlyStartupScreens happen before the engine is fully initialized and block engine initialization before they finish.
    //           this means they have to forgo even the most basic of engine features like UObject support, as they are displayed before those systems are initialized.
    virtual EPreLoadScreenTypes GetPreLoadScreenType() const override { return EPreLoadScreenTypes::EngineLoadingScreen; }

    virtual void SetEngineLoadingFinished(bool IsEngineLoadingFinished) override { bIsEngineLoadingFinished = IsEngineLoadingFinished; }

	// PreLoadScreens not using this functionality should return NAME_None
	virtual FName GetPreLoadScreenTag() const override { return NAME_None; }

    PRELOADSCREEN_API virtual void CleanUp() override;

    //Default behavior is just to see if we have an active widget. Should really overload with our own behavior to see if we are done displaying
    PRELOADSCREEN_API virtual bool IsDone() const override;

public:
    FPreLoadScreenBase()
        : bIsEngineLoadingFinished(false)
    {}

    virtual ~FPreLoadScreenBase() override = default;

    //Handles constructing a FPreLoadSettingsContainerBase with the 
    PRELOADSCREEN_API virtual void InitSettingsFromConfig(const FString& ConfigFileName);

    //Set what plugin is creating this PreLoadScreenBase. Used to make file paths relative to that plugin as well as
    //determining . Used for converting locations for content to be relative to the plugin calling us
    virtual void SetPluginName(const FString& PluginNameIn) { PluginName = PluginNameIn; }

protected:
    TWeakPtr<SWindow> OwningWindow;

    TAtomic<bool> bIsEngineLoadingFinished;
private:
    
    //The name of the Plugin creating this FPreLoadScreenBase.
    //Important: Should be set before Initting settings from Config!
    FString PluginName;
};
