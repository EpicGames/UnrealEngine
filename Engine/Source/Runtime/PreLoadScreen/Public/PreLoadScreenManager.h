// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreLoadScreen.h"

#include "CoreMinimal.h"

#include "Widgets/SVirtualWindow.h"
#include "Widgets/SWindow.h"

#include "TickableObjectRenderThread.h"

#include "Containers/Ticker.h"

#include "PreLoadSlateThreading.h"


// Class that handles storing all registered PreLoadScreens and Playing/Stopping them
class FPreLoadScreenManager
{
public:
    //Gets the single instance of this settings object. Also creates it if needed
    static PRELOADSCREEN_API FPreLoadScreenManager* Get();
    static PRELOADSCREEN_API void Create();
    static PRELOADSCREEN_API void Destroy();

    PRELOADSCREEN_API void Initialize(FSlateRenderer& InSlateRenderer);

    PRELOADSCREEN_API void RegisterPreLoadScreen(const TSharedPtr<IPreLoadScreen>& PreLoadScreen);
    PRELOADSCREEN_API void UnRegisterPreLoadScreen(const TSharedPtr<IPreLoadScreen>& PreLoadScreen);

    /**
	 * Plays the first found PreLoadScreen that matches the bEarlyPreLoadScreen setting passed in.
	 * @returns false if no PreLoadScreen with that type has been registered.
	 */
    PRELOADSCREEN_API bool PlayFirstPreLoadScreen(EPreLoadScreenTypes PreLoadScreenTypeToPlay);

	/**
	 * Plays the PreLoadScreen with a tag that matches InTag
	 * @returns false if no PreLoadScreen with that tag has been registered.
	 */
	PRELOADSCREEN_API bool PlayPreLoadScreenWithTag(FName InTag);
    
	PRELOADSCREEN_API void StopPreLoadScreen();
	PRELOADSCREEN_API void WaitForEngineLoadingScreenToFinish();

    PRELOADSCREEN_API void PassPreLoadScreenWindowBackToGame() const;

    PRELOADSCREEN_API bool IsUsingMainWindow() const;

    PRELOADSCREEN_API TSharedPtr<SWindow> GetRenderWindow();

    PRELOADSCREEN_API bool HasRegisteredPreLoadScreenType(EPreLoadScreenTypes PreLoadScreenTypeToCheck) const;
    PRELOADSCREEN_API bool HasActivePreLoadScreenType(EPreLoadScreenTypes PreLoadScreenTypeToCheck) const;
    PRELOADSCREEN_API bool HasValidActivePreLoadScreen() const;


    PRELOADSCREEN_API void SetEngineLoadingComplete(bool IsEngineLoadingFinished = true);
    bool IsEngineLoadingComplete() const { return bIsEngineLoadingComplete; }

    static PRELOADSCREEN_API void EnableRendering(bool bEnabled);
    static PRELOADSCREEN_API bool ArePreLoadScreensEnabled();
    
    // Callback for handling cleaning up any resources you would like to remove after the PreLoadScreenManager cleans up
    // Not needed for PreLoadScreens as those have a separate CleanUp method called.
    DECLARE_MULTICAST_DELEGATE(FOnPreLoadScreenManagerCleanUp);
    FOnPreLoadScreenManagerCleanUp OnPreLoadScreenManagerCleanUp;

	DECLARE_MULTICAST_DELEGATE_OneParam(FIsPreloadScreenResponsibleForRenderingMultiDelegate, bool);
	FIsPreloadScreenResponsibleForRenderingMultiDelegate IsResponsibleForRenderingDelegate;

protected:
    //Default constructor. We don't want other classes to make these. Should just rely on Get()
    PRELOADSCREEN_API FPreLoadScreenManager();

	~FPreLoadScreenManager() = default;

	PRELOADSCREEN_API void PlayPreLoadScreenAtIndex(int32 Index);

    PRELOADSCREEN_API void BeginPlay();

    /*** These functions describe the flow for an EarlyPreLoadScreen where everything is blocking waiting on a call to StopPreLoadScreen ***/
    PRELOADSCREEN_API void HandleEarlyStartupPlay();
    //Separate tick that handles 
    PRELOADSCREEN_API void EarlyPlayFrameTick();
    PRELOADSCREEN_API void GameLogicFrameTick();
	PRELOADSCREEN_API void EarlyPlayRenderFrameTick();
	PRELOADSCREEN_API bool HasActivePreLoadScreenTypeForEarlyStartup() const;

	PRELOADSCREEN_API void PlatformSpecificGameLogicFrameTick();

	//Creates a tick on the Render Thread that we run every
	static PRELOADSCREEN_API void StaticRenderTick_RenderThread(FRHICommandListImmediate& RHICmdList);
	PRELOADSCREEN_API void RenderTick_RenderThread(FRHICommandListImmediate& RHICmdList);

    /*** These functions describe how everything is handled during an non-Early PreLoadPlay. Everything is handled asynchronously in this case with a standalone renderer ***/
	PRELOADSCREEN_API void HandleEngineLoadingPlay();

	/*** These functions describe the flow for showing an CustomSplashScreen ***/
	PRELOADSCREEN_API void HandleCustomSplashScreenPlay();

    PRELOADSCREEN_API IPreLoadScreen* GetActivePreLoadScreen();
    PRELOADSCREEN_API const IPreLoadScreen* GetActivePreLoadScreen() const;

	PRELOADSCREEN_API void HandleStopPreLoadScreen();
	PRELOADSCREEN_API void CleanUpResources();

	static void OnRequestExit();

	//Singleton Instance
	struct FPreLoadScreenManagerDelete
	{
		void operator()(FPreLoadScreenManager* Ptr) const
		{
			delete Ptr;
		}
	};
	static PRELOADSCREEN_API TUniquePtr<FPreLoadScreenManager, FPreLoadScreenManagerDelete> Instance;
	
	static PRELOADSCREEN_API std::atomic<bool> bRenderingEnabled;

	static FCriticalSection ActivePreloadScreenCriticalSection;
	static TWeakPtr<IPreLoadScreen> ActivePreloadScreen;

	TArray<TSharedPtr<IPreLoadScreen>> PreLoadScreens;

    int32 ActivePreLoadScreenIndex;
    double LastTickTime;

    /** Widget renderer used to tick and paint windows in a thread safe way */
    TSharedPtr<FPreLoadSlateWidgetRenderer> WidgetRenderer;

    /** The window that the loading screen resides in */
    TWeakPtr<class SWindow> MainWindow;

    /** Virtual window that we render to instead of the main slate window (for thread safety).  Shares only the same backbuffer as the main window */
    TSharedPtr<class SVirtualWindow> VirtualRenderWindow;

    bool bInitialized;

    /** The threading mechanism with which we handle running slate on another thread */
	FPreLoadScreenSlateSynchMechanism* SyncMechanism;
	friend FPreLoadScreenSlateSynchMechanism;

	bool bIsResponsibleForRendering;
	bool bHasRenderPreLoadScreenFrame_RenderThread;

    double LastRenderTickTime;

    float OriginalSlateSleepVariableValue;
    bool bIsEngineLoadingComplete;

private:
#if PLATFORM_ANDROID
	PRELOADSCREEN_API void Android_PlatformSpecificGameLogicFrameTick();
#endif //PLATFORM_ANDROID

#if PLATFORM_IOS
	PRELOADSCREEN_API void IOS_PlatformSpecificGameLogicFrameTick();
#endif //PLATFORM_IOS
};
