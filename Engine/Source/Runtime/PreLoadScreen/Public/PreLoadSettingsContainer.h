// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderDeferredCleanup.h"
#include "UObject/GCObject.h"

struct FSlateDynamicImageBrush;
struct FCompositeFont;
struct FStandaloneCompositeFont;

//This is a helper class that we use to hold values we parse from the .ini. Clean way to access things like dynamic image brushes / fonts / etc used in our UI that
//we want to be somewhat data driven but we can't rely on UObject support to implement(as the PreLoad stuff happens too early for UObject support)
//This lets us set easy to change values in our .ini that are parsed at runtime and stored in this container
class FPreLoadSettingsContainerBase : public FDeferredCleanupInterface, public FGCObject
{
public:

    //Helper struct to store groups of things we want to display together in the UI so that we can parse it easily in the .ini. 
    //IE: Show this background, with this text at this font size
    struct FScreenGroupingBase
    {
    public:
        
        FString ScreenBackgroundIdentifer;
        FString TextIdentifier;
        float FontSize;

        FScreenGroupingBase(const FString& ScreenBackgroundIdentifierIn, const FString& TextIdentifierIn, float FontSizeIn)
            : ScreenBackgroundIdentifer(ScreenBackgroundIdentifierIn)
            , TextIdentifier(TextIdentifierIn)
            , FontSize(FontSizeIn)
        {}
    };

	//This is a listing of ScreenGroupings (stored by Identifier) that should be displayed in this order during a particular LoadingGroup.
	//@TODO: TRoss, possible to move these loading groups into their own DeferreedCleanupInterface instead of the entire container being the DeferredCleanupInterface,
	//if we want to support unloading them. For right now though, we mostly just care about loading them selectively and are ok with keeping them in memory until we clean everything up.
	struct FScreenOrderByLoadingGroup
	{
	public:
		TArray<FName> ScreenGroupings;

		FScreenOrderByLoadingGroup()
			: ScreenGroupings ()
		{}
	};

	//Helper struct  to store information required to construct a CustomSlateImageBrush. Parsed from our .ini
	struct FCustomBrushDefine
	{
	public:
		FString BrushIdentifier;
		FString FilePath;
		FVector2D Size;

		FCustomBrushDefine(const FString& BrushIdentifierIn, const FString& FilePathIn, FVector2D SizeIn)
			: BrushIdentifier(BrushIdentifierIn)
			, FilePath(FilePathIn)
			, Size(SizeIn)
		{}
	};

	//Helper struct to store all BrushDefines we need to load for a given BrushLoadingGroup
	struct FCustomBrushLoadingGroup
	{
	public:
		TArray<FCustomBrushDefine> CustomBrushDefinesToLoad;
	};

public:

    static FPreLoadSettingsContainerBase& Get()
    {
        if (Instance == nullptr)
        {
            Instance = new FPreLoadSettingsContainerBase();
        }

        return *Instance;
    }

    static void Destroy()
    {
        if (Instance)
        {
            delete Instance;
            Instance = nullptr;
        }
    }

private:

    FPreLoadSettingsContainerBase() 
		: CurrentLoadGroup(NAME_None)
    {
		bShouldLoadBrushes = true;
		HasCreatedSystemFontFile = false;
    }

    PRELOADSCREEN_API virtual ~FPreLoadSettingsContainerBase();

public:

	//~ Begin FGCObject interface
	PRELOADSCREEN_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FPreLoadSettingsContainerBase"); }
	//~ End FGCObject interface

    PRELOADSCREEN_API virtual const FSlateDynamicImageBrush* GetBrush(const FString& Identifier);
    PRELOADSCREEN_API virtual FText GetLocalizedText(const FString& Identifier);
    PRELOADSCREEN_API virtual TSharedPtr<FCompositeFont> GetFont(const FString& Identifier);
    PRELOADSCREEN_API virtual FScreenGroupingBase* GetScreenGrouping(const FString& Identifier);

    int GetNumScreenGroupings() const { return ScreenGroupings.Num(); }

	PRELOADSCREEN_API virtual const FScreenGroupingBase* GetScreenAtIndex(int index) const;
	PRELOADSCREEN_API virtual bool IsValidScreenIndex(int index) const;

    PRELOADSCREEN_API virtual void CreateCustomSlateImageBrush(const FString& Identifier, const FString& TexturePath, const FVector2D& ImageDimensions);
    PRELOADSCREEN_API virtual void AddLocalizedText(const FString& Identifier, FText LocalizedText);
    PRELOADSCREEN_API virtual void AddScreenGrouping(const FString& Identifier, FScreenGroupingBase& ScreenGrouping);
    
    //Maps the given font file to the given language and stores it under the FontIdentifier.
    //Identifier maps the entire CompositeFont, so if you want to add multiple fonts  for multiple languages, just store them all under the same identifer
    PRELOADSCREEN_API virtual void BuildCustomFont(const FString& FontIdentifier, const FString& Language, const FString& FilePath);
	PRELOADSCREEN_API virtual bool BuildSystemFontFile();
	PRELOADSCREEN_API virtual const FString GetSystemFontFilePath() const;

    //Helper functions that parse a .ini config entry and call the appropriate create function to 
    PRELOADSCREEN_API virtual void ParseBrushConfigEntry(const FString& BrushConfigEntry);
    PRELOADSCREEN_API virtual void ParseFontConfigEntry(const FString&  SplitConfigEntry);
    PRELOADSCREEN_API virtual void ParseLocalizedTextConfigString(const FString&  SplitConfigEntry);
    PRELOADSCREEN_API virtual void ParseScreenGroupingConfigString(const FString&  SplitConfigEntry);

	//Helper function to parse all .ini entries for LoadingGroups and ScreenOrder. Do these together so we can assert if
	//we don't find a matching LoadingGroup identifier in the config. Should be run after we parse all screen groupings
	PRELOADSCREEN_API virtual void ParseLoadingGroups(TArray<FString>& LoadingGroupIdentifiers);
	PRELOADSCREEN_API virtual void ParseAllScreenOrderEntries(TArray<FString>& LoadingGroups, TArray<FString>& ScreenOrderEntries);
	PRELOADSCREEN_API virtual void ParseScreenOrderConfigString(const FString& ScreenOrderEntry);

    //Sets the PluginContent dir so that when parsing config entries we can accept plugin relative file paths
    virtual void SetPluginContentDir(const FString& PluginContentDirIn) { PluginContentDir = PluginContentDirIn; }

	//Tells the container rather it should actually load image brushes
	PRELOADSCREEN_API virtual void SetShouldLoadBrushes(bool bInShouldLoadBrushes);

    float TimeToDisplayEachBackground;
	float TimeToUpdateDownloadedDataSize;
	float TimeToRecheckNotificationsPermission;
	float TimeToShowNotificationsPermissionModal;
    
	FName GetCurrentLoadGrouping() const { return CurrentLoadGroup; }
	PRELOADSCREEN_API void LoadGrouping(FName Identifier);
	PRELOADSCREEN_API void PerformInitialAssetLoad();

    //Helper function that takes in a file path and tries to reconsile it to be Plugin Specific if applicable.
    //Ensures if file is not found in either Plugin's content dir or the original path
    PRELOADSCREEN_API virtual FString ConvertIfPluginRelativeContentPath(const FString& FilePath);

protected:

    //Helper functions that verify if the supplied .ini config entry is valid to create a resource out of it
    PRELOADSCREEN_API virtual bool IsValidBrushConfig(TArray<FString>& SplitConfigEntry);
    PRELOADSCREEN_API virtual bool IsValidFontConfigString(TArray<FString>& SplitConfigEntry);
    PRELOADSCREEN_API virtual bool IsValidLocalizedTextConfigString(TArray<FString>& SplitConfigEntry);
    PRELOADSCREEN_API virtual bool IsValidScreenGrooupingConfigString(TArray<FString>& SplitConfigEntry);

protected:
	TArray<FString> ParsedLoadingGroupIdentifiers;

	/* Property Storage. Ties FName to a particular resource so we can get it by identifier. */
    TMap<FName, FSlateDynamicImageBrush*> BrushResources;
    TMap<FName, FText> LocalizedTextResources;
    TMap<FName, TSharedPtr<FStandaloneCompositeFont>> FontResources;

	TMap<FName, FScreenOrderByLoadingGroup> ScreenOrderByLoadingGroups;
    TMap<FName, FScreenGroupingBase> ScreenGroupings;

	TMap<FName, FCustomBrushLoadingGroup> BrushLoadingGroups;

    //This string is used to make file paths relative to a particular Plugin's content directory when parsing file paths.
    FString PluginContentDir;

	// Rather we should load image brushes
	bool bShouldLoadBrushes;

	FName CurrentLoadGroup;

	bool HasCreatedSystemFontFile;

	//If our Font filepath is set to this, we use the system font instead of a custom font we load in
	static PRELOADSCREEN_API FString UseSystemFontOverride;

	//If we supply no loading groups, use this identifier by default
	static PRELOADSCREEN_API FString DefaultInitialLoadingGroupIdentifier;

    // Singleton Instance -- This is only not a TSharedPtr as it needs to be cleaned up by a deferredcleanup call which directly
    // destroys the underlying object, causing a SharedPtr crash at shutdown.
    static PRELOADSCREEN_API FPreLoadSettingsContainerBase* Instance;
};
