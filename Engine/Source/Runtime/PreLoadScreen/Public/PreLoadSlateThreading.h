// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HAL/Runnable.h"
#include "Misc/ScopeLock.h"
#include "HAL/PlatformApplicationMisc.h"

#include "RHI.h"
#include "RHIResources.h"

class FEvent;
class FRunnableThread;
class FHittestGrid;
class FSlateRenderer;
class SVirtualWindow;
class SWindow;

/**
* The Slate thread is simply run on a worker thread.
* Slate is run on another thread because the game thread (where Slate is usually run)
* is blocked loading things. Slate is very modular, which makes it very easy to run on another
* thread with no adverse effects.
* It does not enqueue render commands, because the RHI is not thread safe. Thus, it waits to
* enqueue render commands until the render thread tickables ticks, and then it calls them there.
*/
class  FPreLoadScreenSlateThreadTask : public FRunnable
{
public:
    FPreLoadScreenSlateThreadTask(class FPreLoadScreenSlateSynchMechanism& InSyncMechanism)
        : SyncMechanism(&InSyncMechanism)
    {
    }

    //~ Begin FRunnable interface
    virtual bool Init() override;
    virtual uint32 Run() override;
	virtual void Exit() override;
	//~ End FRunnable interface

private:
    /** Hold a handle to our parent sync mechanism which handles all of our threading locks */
    class FPreLoadScreenSlateSynchMechanism* SyncMechanism;
};

class FPreLoadSlateWidgetRenderer
{
public:
	PRELOADSCREEN_API FPreLoadSlateWidgetRenderer(TSharedPtr<SWindow> InMainWindow, TSharedPtr<SVirtualWindow> InVirtualRenderWindowWindow, FSlateRenderer* InRenderer);

	PRELOADSCREEN_API void DrawWindow(float DeltaTime);

	SWindow* GetMainWindow_GameThread() const { return MainWindow; }

private:
	/** The actual window content will be drawn to */
	/** Note: This is raw as we SWindows registered with SlateApplication are not thread safe */
	SWindow* MainWindow;

	/** Virtual window that we render to instead of the main slate window (for thread safety).  Shares only the same backbuffer as the main window */
	TSharedRef<SVirtualWindow> VirtualRenderWindow;

	TSharedPtr<FHittestGrid> HittestGrid;

	FSlateRenderer* SlateRenderer;

	FViewportRHIRef ViewportRHI;
};


/**
 * This class will handle all the nasty bits about running Slate on a separate thread
 * and then trying to sync it up with the game thread and the render thread simultaneously
 */
class FPreLoadScreenSlateSynchMechanism
{
public:
    PRELOADSCREEN_API FPreLoadScreenSlateSynchMechanism(TSharedPtr<FPreLoadSlateWidgetRenderer, ESPMode::ThreadSafe> InWidgetRenderer);
    PRELOADSCREEN_API ~FPreLoadScreenSlateSynchMechanism();

	FPreLoadScreenSlateSynchMechanism() = delete;
	FPreLoadScreenSlateSynchMechanism(const FPreLoadScreenSlateSynchMechanism&) = delete;
	FPreLoadScreenSlateSynchMechanism& operator=(const FPreLoadScreenSlateSynchMechanism&) = delete;

    /** Sets up the locks in their proper initial state for running */
    PRELOADSCREEN_API void Initialize();

    /** Cleans up the slate thread */
    PRELOADSCREEN_API void DestroySlateThread();

    /** Handles the counter to determine if the slate thread should keep running */
    PRELOADSCREEN_API bool IsSlateMainLoopRunning_AnyThread() const;

private:
	/** Notified when a SWindow is being destroyed */
	PRELOADSCREEN_API void HandleWindowBeingDestroyed(const SWindow& WindowBeingDestroyed);

	/** The main loop to be run from the Slate thread */
	PRELOADSCREEN_API void RunMainLoop_SlateThread();

    /** This counter handles running the main loop of the slate thread */
    TAtomic<bool> bIsRunningSlateMainLoop;

    /** This counter is used to generate a unique id for each new instance of the loading thread */
    static PRELOADSCREEN_API TAtomic<int32> LoadingThreadInstanceCounter;

    /** The worker thread that will become the Slate thread */
    FRunnableThread* SlateLoadingThread;
    FRunnable* SlateRunnableTask;
	FEvent* SleepEvent;

    TSharedPtr<FPreLoadSlateWidgetRenderer, ESPMode::ThreadSafe> WidgetRenderer;

	friend FPreLoadScreenSlateThreadTask;
};
