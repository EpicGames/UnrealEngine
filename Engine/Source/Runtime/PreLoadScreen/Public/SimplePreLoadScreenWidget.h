// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"

#include "RenderingThread.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SDPIScaler.h"

//Widget that displays a very simple version of a FPreLoadScreen UI that just includes a background and localized text together.
//Rotates through the PreLoadScreens in the same order they are in the FPreLoadSettingsContainerBase. Uses the TimeToDisplayEachBackground variable to determine how long
//to display each screen before rotating. Loops back when finished.
class SSimplePreLoadScreenWidget : public SCompoundWidget
{
public:
    SLATE_BEGIN_ARGS(SSimplePreLoadScreenWidget) {}
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs);
    
    //Handles updating the background every X seconds
    virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
    //Loops to the next background image
    virtual void UpdateBackgroundImage();

    //Not used in the default simple implementation
    virtual float GetDPIScale() const { return 1.0f; };

    const FSlateBrush* GetCurrentBackgroundImage() const;
    FText GetCurrentScreenText() const;
    FSlateFontInfo GetTextFont() const;
    FSlateFontInfo GetFontInfo(const FString& FontName, float FontSize) const;

    int CurrentPreLoadScreenIndex;
    float TimeToDisplayEachBackground;
    float TimeSinceLastBackgroundUpdate;

    static FCriticalSection BackgroundImageCrit;
    static int CurrentBackgroundImage;

};